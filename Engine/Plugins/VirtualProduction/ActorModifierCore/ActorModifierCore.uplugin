{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Actor Modifier Core",
	"Description": "Use modifier objects on actors to apply a custom behavior",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"PlatformAllowList": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "ActorModifierCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorModifierCoreBlueprint",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorModifierCoreEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "OperatorStack",
			"Enabled": true
		}
	]
}