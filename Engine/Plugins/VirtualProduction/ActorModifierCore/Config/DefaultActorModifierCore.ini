[CoreRedirects]
+ClassRedirects=(OldName="/Script/TagCollection.TagCollectionModifierSharedActor",NewName="/Script/ActorModifierCore.ActorModifierCoreSharedActor")
+ClassRedirects=(OldName="/Script/TagCollection.TagCollectionModifierBase",NewName="/Script/ActorModifierCore.ActorModifierCoreBase")
+ClassRedirects=(OldName="/Script/TagCollection.TagCollectionModifierComponent",NewName="/Script/ActorModifierCore.ActorModifierCoreComponent")
+ClassRedirects=(OldName="/Script/TagCollection.TagCollectionModifierSharedObject",NewName="/Script/ActorModifierCore.ActorModifierCoreSharedObject")
+ClassRedirects=(OldName="/Script/TagCollection.TagCollectionModifierStack",NewName="/Script/ActorModifierCore.ActorModifierCoreStack")
+ClassRedirects=(OldName="/Script/TagCollectionEditor.TagCollectionModifierStackCustomization",NewName="/Script/ActorModifierCoreEditor.ActorModifierCoreEditorStackCustomization")
+ClassRedirects=(OldName="/Script/TagCollectionEditor.TagCollectionModifierExtensionSubsystem",NewName="/Script/ActorModifierCoreEditor.ActorModifierCoreEditorSubsystem")
+StructRedirects=(OldName="/Script/TagCollectionEditor.TagCollectionModifierPropertiesWrapper",NewName="/Script/ActorModifierCoreEditor.ActorModifierCoreEditorPropertiesWrapper")
+FunctionRedirects=(OldName="/Script/ActorModifierCore.ActorModifierCoreLibrary.FindModifierOfClass",NewName="/Script/ActorModifierCore.ActorModifierCoreLibrary.FindModifierByClass")