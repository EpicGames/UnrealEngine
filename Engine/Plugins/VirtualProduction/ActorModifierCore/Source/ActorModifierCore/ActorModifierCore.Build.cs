// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierCore : ModuleRules
{
	public ActorModifierCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Slate",
				"SlateCore",
				"TraceLog"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd"
			});
		}
	}
}
