// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ActorModifierCoreLog.h"

DEFINE_LOG_CATEGORY(LogActorModifierCore)

IMPLEMENT_MODULE(FDefaultModuleImpl, ActorModifierCore)