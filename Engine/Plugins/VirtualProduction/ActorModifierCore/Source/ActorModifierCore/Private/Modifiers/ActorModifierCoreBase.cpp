// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreBase.h"

#include "ActorModifierCoreLog.h"
#include "Async/TaskGraphInterfaces.h"
#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreBase, Log, All);

UActorModifierCoreSharedObject* UActorModifierCoreBase::GetShared(TSubclassOf<UActorModifierCoreSharedObject> InClass, bool bInCreateIfNone) const
{
	UActorModifierCoreSharedObject* SharedObject = nullptr;

	if (!InClass.Get())
	{
		return SharedObject;
	}

	const AActor* ActorModified = GetModifiedActor();
	if (!ActorModified)
	{
		return SharedObject;
	}

	if (const UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		ULevel* Level = ActorModified->GetLevel();
		SharedObject = Subsystem->GetModifierSharedObject(Level, InClass, bInCreateIfNone);
	}

	if (bInCreateIfNone && !SharedObject)
	{
		UE_LOGF(LogActorModifierCore, Error, "[%ls][%ls][%ls] Failed to create and retrieve the specific shared object : %ls"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, *InClass.Get()->GetName());
	}

	return SharedObject;
}

void UActorModifierCoreBase::Apply()
{
	checkNoEntry();
	Fail(FText::FromString(TEXT("Apply not implemented")));
}

void UActorModifierCoreBase::Next()
{
	auto ExecuteNext = [this]()
	{
		if (UActorModifierCoreStack* ModifierStack = GetModifierStack())
		{
			if (!bModifierIdle && ModifierStack->ExecutionTask.GetCurrentModifier() == this)
			{
				// Success
				Status = FActorModifierCoreStatus(EActorModifierCoreStatus::Success, FText::GetEmpty());
				ModifierStack->ExecutionTask.Next();
			}
			else
			{
				checkf(false, TEXT("Next is called again after execution is done"))
			}
		}
	};

	if (!IsInGameThread())
	{
		TWeakObjectPtr<UActorModifierCoreBase> ThisWeak(this);
		FFunctionGraphTask::CreateAndDispatchWhenReady([ThisWeak, ExecuteNext]()
		{
			const UActorModifierCoreBase* This = ThisWeak.Get();

			if (!This)
			{
				return;
			}

			ExecuteNext();
		}, {}, nullptr, ENamedThreads::GameThread);
	}
	else
	{
		ExecuteNext();
	}
}

void UActorModifierCoreBase::Fail(const FText& InFailReason)
{
	auto ExecuteFail = [this, InFailReason]()
	{
		if (UActorModifierCoreStack* ModifierStack = GetModifierStack())
		{
			if (!bModifierIdle && ModifierStack->ExecutionTask.GetCurrentModifier() == this)
			{
				// Provide a valid failing reason
				checkf(!InFailReason.IsEmpty(), TEXT("Provide a valid fail reason"))

				// Success
				Status = FActorModifierCoreStatus(EActorModifierCoreStatus::Error, InFailReason);
				ModifierStack->ExecutionTask.Fail();
			}
			else
			{
				checkf(false, TEXT("Fail is called again after execution is done"));
			}
		}
	};
	
	if (!IsInGameThread())
	{
		TWeakObjectPtr<UActorModifierCoreBase> ThisWeak(this);
		FFunctionGraphTask::CreateAndDispatchWhenReady([ThisWeak, ExecuteFail]()
		{
			const UActorModifierCoreBase* This = ThisWeak.Get();

			if (!This)
			{
				return;
			}

			ExecuteFail();
		}, {}, nullptr, ENamedThreads::GameThread);
	}
	else
	{
		ExecuteFail();
	}
}

void UActorModifierCoreBase::Unapply()
{
	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Unapplying modifier"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString());

	// only restore if this modifier was already applied previously
	if (bModifierApplied)
	{
		RestorePreState();

		bModifierApplied = false;
	}
}

void UActorModifierCoreBase::OnModifierDirty(UActorModifierCoreBase* DirtyModifier, bool bExecute)
{
	if (UActorModifierCoreStack* ModifierStack = GetModifierStack())
	{
		ModifierStack->OnModifierDirty(DirtyModifier, bExecute);
	}
}

const FActorModifierCoreMetadata& UActorModifierCoreBase::GetModifierMetadata() const
{
	return *Metadata;
}

FName UActorModifierCoreBase::GetModifierName() const
{
	return Metadata->GetName();
}

FName UActorModifierCoreBase::GetModifierCategory() const
{
	return Metadata->GetCategory();
}

bool UActorModifierCoreBase::IsModifierStack() const
{
	return Metadata->IsStack();
}

AActor* UActorModifierCoreBase::GetModifiedActor() const
{
	if (!ModifiedActor.IsValid())
	{
		const_cast<UActorModifierCoreBase*>(this)->ModifiedActor = GetTypedOuter<AActor>();
	}
	return ModifiedActor.Get();
}

UActorModifierCoreStack* UActorModifierCoreBase::GetModifierStack() const
{
	return GetTypedOuter<UActorModifierCoreStack>();
}

UActorModifierCoreStack* UActorModifierCoreBase::GetRootModifierStack() const
{
	// we are not the root stack
	if (const UActorModifierCoreStack* Stack = GetModifierStack())
	{
		return Stack->GetRootModifierStack();
	}

	// we are the root stack
	UActorModifierCoreBase* This = const_cast<UActorModifierCoreBase*>(this);
	return Cast<UActorModifierCoreStack>(This);
}

UActorModifierCoreComponent* UActorModifierCoreBase::GetModifierComponent() const
{
	return GetTypedOuter<UActorModifierCoreComponent>();
}

const UActorModifierCoreBase* UActorModifierCoreBase::GetPreviousModifier() const
{
	const UActorModifierCoreBase* PreviousNameModifier = nullptr;
	if (const UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		RootStack->ProcessFunction([&PreviousNameModifier, this](const UActorModifierCoreBase* InModifier)->bool
		{
			// stop when we are the current modifier
			if (InModifier == this)
			{
				return false;
			}

			PreviousNameModifier = InModifier;

			return true;
		});
	}
	return PreviousNameModifier;
}

const UActorModifierCoreBase* UActorModifierCoreBase::GetNextModifier() const
{
	const UActorModifierCoreBase* NextNameModifier = nullptr;
	if (const UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		bool bStartSearch = false;
		RootStack->ProcessFunction([&NextNameModifier, &bStartSearch, this](const UActorModifierCoreBase* InModifier)->bool
		{
			if (bStartSearch)
			{
				NextNameModifier = InModifier;

				// stop we have found our next modifier
				return false;
			}

			if (InModifier == this)
			{
				bStartSearch = true;
			}

			// keep going
			return true;
		});
	}
	return NextNameModifier;
}

const UActorModifierCoreBase* UActorModifierCoreBase::GetPreviousNameModifier(const FName& InModifierName) const
{
	const UActorModifierCoreBase* PreviousNameModifier = nullptr;
	if (const UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		RootStack->ProcessFunction([&PreviousNameModifier, InModifierName, this](const UActorModifierCoreBase* InModifier)->bool
		{
			// stop when we are the current modifier
			if (InModifier == this)
			{
				return false;
			}

			// is it the name we are looking for
			if (InModifier->GetModifierName() == InModifierName)
			{
				PreviousNameModifier = InModifier;
			}

			// keep going since we want the closest one
			return true;
		});
	}
	return PreviousNameModifier;
}

const UActorModifierCoreBase* UActorModifierCoreBase::GetNextNameModifier(const FName& InModifierName) const
{
	const UActorModifierCoreBase* NextNameModifier = nullptr;
	if (const UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		bool bStartSearch = false;
		RootStack->ProcessFunction([&NextNameModifier, &bStartSearch, InModifierName, this](const UActorModifierCoreBase* InModifier)->bool
		{
			if (bStartSearch)
			{
				// is this the name we are looking for
				if (InModifier->GetModifierName() == InModifierName)
				{
					NextNameModifier = InModifier;

					// stop we have found our next modifier
					return false;
				}
			}

			if (InModifier == this)
			{
				bStartSearch = true;
			}

			// keep going
			return true;
		});
	}
	return NextNameModifier;
}

bool UActorModifierCoreBase::CanMarkModifierDirty() const
{
	const UActorModifierCoreStack* Stack = GetRootModifierStack();

	// Do not mark dirty if stack is in execution
	if (!Stack || !Stack->IsModifierIdle() || !Stack->IsModifierInitialized() || !IsModifierInitialized())
	{
		return false;
	}

	// When modifier is disabled but applied, we need to mark it dirty for it to be restored
	if (!IsModifierEnabled() && !IsModifierApplied())
	{
		return false;
	}

	// Can't mark modifier dirty. It's already dirtied.
	return !bModifierDirty;
}

bool UActorModifierCoreBase::MarkModifierDirty(bool bExecute)
{
	if (CanMarkModifierDirty() || bExecute)
	{
		bModifierDirty = true;
		OnModifierDirty(this, bExecute);
		return true;
	}
	return false;
}

bool UActorModifierCoreBase::IsModifierEnabled() const
{
	if (!bModifierEnabled)
	{
		return false;
	}
	const UActorModifierCoreStack* Stack = GetModifierStack();
	return !Stack || (Stack->IsModifierEnabled() && !Stack->IsModifierStackFrozen());
}

void UActorModifierCoreBase::ProcessLockFunction(TFunctionRef<void()> InFunction)
{
	LockModifierExecution();
	InFunction();
	UnlockModifierExecution();
}

void UActorModifierCoreBase::LockModifierExecution()
{
	if (!bModifierExecutionLocked)
	{
		bModifierExecutionLocked = true;

		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Locking modifier execution"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());
	}
}

void UActorModifierCoreBase::UnlockModifierExecution()
{
	if (bModifierExecutionLocked)
	{
		bModifierExecutionLocked = false;

		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Unlocking modifier execution"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());

		if (IsModifierDirty())
		{
			MarkModifierDirty(true);
		}
	}
}

void UActorModifierCoreBase::AddExtensionInternal(const FName& InExtensionType, TSharedPtr<FActorModifierCoreExtension> InExtension)
{
	if (!InExtension.IsValid())
	{
		return;
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Adding modifier extension %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *InExtensionType.ToString());

	ModifierExtensions.Emplace(InExtensionType, InExtension);

	InExtension->ConstructInternal(this, InExtensionType);

	if (bModifierEnabled)
	{
		InExtension->EnableExtension(EActorModifierCoreEnableReason::User);
	}
}

void UActorModifierCoreBase::SetModifierEnabled(bool bInEnabled)
{
	if (bModifierEnabled == bInEnabled)
	{
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	bModifierEnabled = bInEnabled;
	OnModifierEnabledChanged(/** Execute */true);
}

bool UActorModifierCoreBase::IsModifierProfiling() const
{
	if (const UActorModifierCoreStack* Stack = GetRootModifierStack())
	{
		return Stack->bModifierProfiling;
	}

	return false;
}

bool UActorModifierCoreBase::ProcessFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	return InFunction(this);
}

void UActorModifierCoreBase::DeferInitializeModifier()
{
	if (IsModifierInitialized())
	{
		return;
	}

	// Begin batch operation to avoid updating every time a modifier is loaded
	UActorModifierCoreStack* Stack = GetRootModifierStack();

	if (!Stack)
	{
		return;
	}

	if (!Stack->IsModifierExecutionLocked() && !Stack->IsModifierStackInitialized())
	{
		Stack->LockModifierExecution();
	}

	// Bind to world delegate, tick will be called when all actors have been loaded, unbind when actors have been loaded
	FWorldDelegates::OnWorldTickEnd.RemoveAll(this);
	FWorldDelegates::OnWorldTickEnd.AddUObject(this, &UActorModifierCoreBase::PostModifierWorldLoad);
}

UActorModifierCoreBase::UActorModifierCoreBase()
{
	// Copy metadata from CDO
	if (!IsTemplate())
	{
		const UActorModifierCoreBase* CDO = GetClass()->GetDefaultObject<UActorModifierCoreBase>();
		Metadata = CDO->Metadata;
	}
}

void UActorModifierCoreBase::PostLoad()
{
	Super::PostLoad();

	const AActor* OwningActor = GetModifiedActor();
	const UObject* Outer = GetOuter();
	const UActorModifierCoreComponent* OwningComponent = OwningActor ? OwningActor->FindComponentByClass<UActorModifierCoreComponent>() : nullptr;

	if (OwningActor
		&& OwningComponent
		&& Outer
		&& Outer->IsA<AActor>())
	{
		constexpr int32 RenameFlags = REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch;

		// Migrate modifiers to component stack instead and discard this stack
		if (const UActorModifierCoreStack* ThisStack = Cast<UActorModifierCoreStack>(this))
		{
			UActorModifierCoreStack* ComponentStack = OwningComponent->ModifierStack;

			if (ComponentStack && ComponentStack != this)
			{
				const FString ThisStackName = GetName();
				const EObjectFlags ThisStackFlags = GetFlags();

				UE_LOGF(LogActorModifierCore, Log, "[%ls][%ls][%ls] Modifier stack migrated to component stack %ls %ls with %i modifiers"
					, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
					, *GetNameSafe(GetClass())
					, *GetModifierName().ToString()
					, *OwningComponent->GetName()
					, *ComponentStack->GetName()
					, ThisStack->Modifiers.Num());

				Rename(nullptr, GetTransientPackage(), RenameFlags);

				ComponentStack->Rename(*ThisStackName, nullptr, RenameFlags);
				ComponentStack->Modifiers = ThisStack->Modifiers;
				ComponentStack->bModifierProfiling = ThisStack->bModifierProfiling;
				ComponentStack->SetFlags(ThisStackFlags);

				return;
			}
		}
		// Change outer of modifier to component stack instead of actor
		else
		{
			const bool bSuccess = Rename(nullptr, OwningComponent->GetModifierStack(), RenameFlags);

			UE_LOGF(LogActorModifierCore, Log, "[%ls][%ls][%ls] Modifier outer renamed to stack %ls : %ls"
				, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
				, *GetNameSafe(GetClass())
				, *GetModifierName().ToString()
				, *OwningComponent->GetModifierStack()->GetName()
				, bSuccess ? TEXT("OK") : TEXT("Fail"));
		}
	}

	DeferInitializeModifier();
}

void UActorModifierCoreBase::PostEditImport()
{
	Super::PostEditImport();

	InitializeModifier(EActorModifierCoreEnableReason::Duplicate);

	// Execute stack to update modifiers after duplication process
	if (IsModifierStack())
	{
		OnModifierEnabledChanged(true);
	}
}

void UActorModifierCoreBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// Only consider non-pie duplication here.
	// Initializing a modifier via PIE will be handled via UActorModifierCoreBase::PostModifierWorldLoad 
	if (DuplicateMode != EDuplicateMode::PIE)
	{
		InitializeModifier(EActorModifierCoreEnableReason::Duplicate);
	}
}

#if WITH_EDITOR
void UActorModifierCoreBase::PreEditUndo()
{
	Super::PreEditUndo();

	if (UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		if (RootStack->IsModifierIdle())
		{
			RootStack->LockModifierExecution();
			RootStack->MarkModifierDirty(false);
			RootStack->RestorePreState();
		}
	}
}

void UActorModifierCoreBase::PostEditUndo()
{
	Super::PostEditUndo();

	UActorModifierCoreStack* ModifierStack = GetModifierStack();

	// is it an undo remove or undo add operation ?
	const bool bModifierInStack = ModifierStack && ModifierStack->Modifiers.Contains(this);
	const bool bStackRegistered = IsA<UActorModifierCoreStack>() && !ModifierStack;
	const bool bModifierValid = bModifierInStack || bStackRegistered;

	if (!bModifierValid)
	{
		UninitializeModifier(EActorModifierCoreDisableReason::Undo);
	}
	else
	{
		InitializeModifier(EActorModifierCoreEnableReason::Undo);
	}

	// refresh the whole stack
	if (UActorModifierCoreStack* InStack = GetRootModifierStack())
	{
		if (InStack->IsModifierStackInitialized())
		{
			InStack->UnlockModifierExecution();
		}
	}
}

void UActorModifierCoreBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName ModifierEnabledName = GET_MEMBER_NAME_CHECKED(UActorModifierCoreBase, bModifierEnabled);

	if (InPropertyChangedEvent.GetMemberPropertyName() == ModifierEnabledName)
	{
		OnModifierEnabledChanged();
	}
}
#endif

FActorModifierCoreExtension* UActorModifierCoreBase::GetExtension(const FName& InExtensionType) const
{
	const TSharedPtr<FActorModifierCoreExtension>* Extension = ModifierExtensions.Find(InExtensionType);
	return Extension ? Extension->Get() : nullptr;
}

bool UActorModifierCoreBase::RemoveExtension(const FName& InExtensionType)
{
	if (const TSharedPtr<FActorModifierCoreExtension>* Extension = ModifierExtensions.Find(InExtensionType))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Removing modifier extension %ls"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, *InExtensionType.ToString());

		if (bModifierEnabled)
		{
			Extension->Get()->DisableExtension(EActorModifierCoreDisableReason::User);
		}

		return ModifierExtensions.Remove(InExtensionType) > 0;
	}

	return false;
}

void UActorModifierCoreBase::LogModifier(const FString& InLog, bool bInForce, EActorModifierCoreStatus InType) const
{
}

void UActorModifierCoreBase::PostModifierCreation(UActorModifierCoreStack* InStack)
{
	// initialize once, called by the subsystem itself
	if (GetModifierStack() == InStack)
	{
		if (UActorModifierCoreBase* CDO = GetClass()->GetDefaultObject<UActorModifierCoreBase>())
		{
			if (!CDO->Metadata.IsValid())
			{
				if (UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
				{
					ModifierSubsystem->RegisterModifierClass(CDO->GetClass(), /** Override*/true);
				}
			}
			
			Metadata = CDO->Metadata;
		}

		ModifiedActor = GetModifiedActor();
		bModifierInitialized = false;
	}
}

void UActorModifierCoreBase::PostModifierCDOCreation()
{
	if (IsTemplate())
	{
		Metadata = MakeShared<FActorModifierCoreMetadata>(this);
		OnModifierCDOSetup(*Metadata);

#if WITH_EDITOR
		// Set a display name if none was provided
		if (Metadata->GetDisplayName().IsEmpty())
		{
			Metadata->SetDisplayName(FText::FromString(FName::NameToDisplayString(Metadata->GetName().ToString(), false)));
		}
#endif
	}
}

void UActorModifierCoreBase::PostModifierWorldLoad(UWorld* InWorld, ELevelTick InType, float InDelta)
{
	const AActor* Actor = GetModifiedActor();

	// Check actor is in the world loaded and does not need post load and is not in async loading
	if (Actor
		&& InWorld == Actor->GetWorld()
		&& !Actor->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading)
		&& !Actor->HasAnyFlags(EObjectFlags::RF_NeedPostLoad)
		&& !Actor->HasAnyFlags(EObjectFlags::RF_NeedPostLoadSubobjects))
	{
		// Check that components are post loaded and ready to be used
		const bool bActorComponentsPostLoaded = ForEachComponent<UActorComponent>([](UActorComponent* InComponent)->bool
		{
			return InComponent
				&& !InComponent->HasAnyFlags(EObjectFlags::RF_NeedPostLoad)
				&& !InComponent->HasAnyFlags(EObjectFlags::RF_NeedPostLoadSubobjects);
		}
		, EActorModifierCoreComponentType::All
		, EActorModifierCoreLookup::Self);

		if (!bActorComponentsPostLoaded)
		{
			return;
		}

		// Remove handle
		FWorldDelegates::OnWorldTickEnd.RemoveAll(this);

		// Initialize now that all actors of world have been post loaded
		InitializeModifier(EActorModifierCoreEnableReason::Load);

		// End batch operation and execute all modifiers at once if all stack is initialized
		if (GetRootModifierStack() == this)
		{
			if (IsModifierExecutionLocked())
			{
				UnlockModifierExecution();
			}
			else
			{
				MarkModifierDirty();
			}
		}
	}
}

void UActorModifierCoreBase::EnableModifier(EActorModifierCoreEnableReason InReason)
{
	OnModifierEnabled(InReason);

	// Enable extensions
	for (const TPair<FName, TSharedPtr<FActorModifierCoreExtension>>& ExtensionPair : ModifierExtensions)
	{
		if (ExtensionPair.Value.IsValid())
		{
			ExtensionPair.Value->EnableExtension(InReason);
		}
	}
}

void UActorModifierCoreBase::DisableModifier(EActorModifierCoreDisableReason InReason)
{
	OnModifierDisabled(InReason);

	// Disable extensions
	for (const TPair<FName, TSharedPtr<FActorModifierCoreExtension>>& ExtensionPair : ModifierExtensions)
	{
		if (ExtensionPair.Value.IsValid())
		{
			ExtensionPair.Value->DisableExtension(InReason);
		}
	}
}

void UActorModifierCoreBase::InitializeModifier(EActorModifierCoreEnableReason InReason)
{
	// is the modifier correctly setup
	if (!bModifierInitialized)
	{
		bModifierInitialized = true;

#if WITH_EDITOR
		// to be able to track property changes and stack updates
		if (!HasAnyFlags(RF_Transactional))
		{
			SetFlags(GetFlags() | RF_Transactional);
		}
#endif

		// set new actor
		ModifiedActor = GetModifiedActor();

		// if the metadata is not initialized, reload it from CDO
		if (!Metadata.IsValid())
		{
			PostModifierCreation(GetModifierStack());

			// Set it to true again since PostModifierCreation sets bModifierInitialized to false
			bModifierInitialized = true;

			// Cannot proceed with invalid metadata
			if (!Metadata.IsValid())
			{
				UE_LOGF(LogActorModifierCoreBase, Fatal, "Invalid modifier metadata for instance of class %ls", *GetClass()->GetName())	
			}
		}

		// Initialize profiler
		if (!Profiler.IsValid())
		{
			Profiler = Metadata->CreateProfilerInstance(this);
		}

		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Initializing modifier with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		// add the modifier to our new actor stack
		OnModifierAdded(InReason);

		// if the original state was enable, enable it
		if (bModifierEnabled)
		{
			EnableModifier(InReason);
		}

		const bool bExecuteModifiers = IsModifierStack();
		MarkModifierDirty(bExecuteModifiers);

		UActorModifierCoreStack::OnModifierAddedDelegate.Broadcast(this, InReason);
	}
}

void UActorModifierCoreBase::UninitializeModifier(EActorModifierCoreDisableReason InReason)
{
	if (bModifierInitialized)
	{
		bModifierInitialized = false;

		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Uninitializing modifier with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		const bool bWasModifierEnabled = bModifierEnabled;

		MarkModifierDirty(false);

		// if modifier is enable we need to disable it first
		if (bModifierEnabled)
		{
			bModifierEnabled = false;
			DisableModifier(InReason);
		}

		// lets remove it now from old actor
		OnModifierRemoved(InReason);

		// set new actor
		ModifiedActor = GetModifiedActor();

		// recover old enabled state
		bModifierEnabled = bWasModifierEnabled;

		UActorModifierCoreStack::OnModifierRemovedDelegate.Broadcast(this, InReason);
	}
}

void UActorModifierCoreBase::OnModifierEnabledChanged(bool bInExecute)
{
	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Modifier %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, bModifierEnabled ? TEXT("enabled") : TEXT("disabled"));

	if (bModifierEnabled)
	{
		EnableModifier(EActorModifierCoreEnableReason::User);
	}
	else
	{
		DisableModifier(EActorModifierCoreDisableReason::User);
	}

	MarkModifierDirty(bInExecute);
}

void UActorModifierCoreBase::BeginModifierExecution()
{
	bModifierIdle = false;

	const UActorModifierCoreStack* Stack = GetRootModifierStack();

	if (IsModifierEnabled()
		&& Stack && Stack->IsModifierEnabled() && Stack->IsModifierProfiling()
		&& Profiler.IsValid())
	{
		Profiler->BeginProfiling();
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Applying Modifier"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString());
}

void UActorModifierCoreBase::EndModifierExecution()
{
	bModifierIdle = true;

	const UActorModifierCoreStack* Stack = GetRootModifierStack();

	if (IsModifierEnabled()
		&& Stack && Stack->IsModifierEnabled() && Stack->IsModifierProfiling()
		&& Profiler.IsValid())
	{
		Profiler->EndProfiling();
	}

	UE_CLOGF(Status.GetStatus() != EActorModifierCoreStatus::Success, LogActorModifierCore, Warning, "[%ls][%ls][%ls] Modifier execution failed due to reason : %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *Status.GetStatusMessage().ToString());
}

