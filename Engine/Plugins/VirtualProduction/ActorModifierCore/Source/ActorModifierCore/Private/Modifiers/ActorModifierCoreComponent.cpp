// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/ActorModifierCoreStack.h"

UActorModifierCoreComponent::UActorModifierCoreComponent()
{
	ModifierStack = CreateDefaultSubobject<UActorModifierCoreStack>(TEXT("ModifierStack"));
	ModifierStack->PostModifierCreation(/** Parentstack */nullptr);

	if (!IsTemplate())
	{
		PrimaryComponentTick.bCanEverTick = true;
		bTickInEditor = true;
	}
}

UActorModifierCoreComponent* UActorModifierCoreComponent::CreateAndExposeComponent(AActor* InParentActor)
{
	if (!InParentActor)
	{
		return nullptr;
	}

	if (UActorModifierCoreComponent* ModifierComponent = InParentActor->FindComponentByClass<UActorModifierCoreComponent>())
	{
		return ModifierComponent;
	}

#if WITH_EDITOR
	InParentActor->Modify();
#endif

	const UClass* const ModifierComponentClass = UActorModifierCoreComponent::StaticClass();

	// Construct the new component and attach as needed
	UActorModifierCoreComponent* const ModifierComponent = NewObject<UActorModifierCoreComponent>(InParentActor
		, ModifierComponentClass
		, MakeUniqueObjectName(InParentActor, ModifierComponentClass, TEXT("ModifierStackComponent"))
		, RF_Transactional);

	// Add to SerializedComponents array so it gets saved
	InParentActor->AddInstanceComponent(ModifierComponent);
	ModifierComponent->OnComponentCreated();
	ModifierComponent->RegisterComponent();

#if WITH_EDITOR
	// Rerun construction scripts
	InParentActor->RerunConstructionScripts();
#endif

	return ModifierComponent;
}

void UActorModifierCoreComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (ModifierStack && !ModifierStack->IsModifierInitialized())
	{
		ModifierStack->InitializeModifier(EActorModifierCoreEnableReason::User);
	}
}

void UActorModifierCoreComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	// Un-initialize the stack linked to this component
	if (ModifierStack)
	{
		ModifierStack->UninitializeModifier(EActorModifierCoreDisableReason::Destroyed);
	}
}

void UActorModifierCoreComponent::PostLoad()
{
	Super::PostLoad();

	// Due to the fact that we replaced stack by sub-object stack, init is needed here as old stack is deleted
	if (ModifierStack && !ModifierStack->IsModifierInitialized())
	{
		ModifierStack->DeferInitializeModifier();
	}
}

#if WITH_EDITOR
void UActorModifierCoreComponent::PostEditUndo()
{
	Super::PostEditUndo();

	if (ModifierStack)
	{
		// call the post edit undo from the stack since it does not get called
		ModifierStack->PostEditUndo();

		// execute stack dirty
		ModifierStack->MarkModifierDirty();
	}
}
#endif

void UActorModifierCoreComponent::TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InThisTickFunction);

	if (ModifierStack)
	{
		ModifierStack->TickModifier(InDeltaTime);
	}
}
