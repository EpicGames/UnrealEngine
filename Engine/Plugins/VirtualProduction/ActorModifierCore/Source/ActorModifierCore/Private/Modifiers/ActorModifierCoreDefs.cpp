// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreDefs.h"

#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#if WITH_EDITOR
#include "Styling/SlateIconFinder.h"
#endif

#define LOCTEXT_NAMESPACE "ActorModifierCoreMetadata"

FActorModifierCoreMetadata::FActorModifierCoreMetadata()
	: bIsStack(false)
#if WITH_EDITOR
	, bHidden(false)
	, Color(FLinearColor::White)
#endif
{
}

FActorModifierCoreMetadata::FActorModifierCoreMetadata(const UActorModifierCoreBase* InModifier)
{
	check(InModifier && InModifier->IsTemplate());

	Class = InModifier->GetClass();
	bIsStack = InModifier->IsA<UActorModifierCoreStack>();

	SetCategory(DefaultCategory);
	SetName(Class->GetFName());
	SetProfilerClass<FActorModifierCoreProfiler>();
	SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return IsValid(InActor);
	});

#if WITH_EDITOR
	Icon = FSlateIconFinder::FindIconForClass(Class);
	if (!Icon.IsSet())
	{
		Icon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());
	}

	Description = Class->GetToolTipText();
	DisplayName = Class->GetDisplayNameText();
	bHidden = Class->HasMetaData("Hidden");
#endif
}

bool FActorModifierCoreMetadata::IsDisallowedAfter(const FName& InModifierName) const
{
	return DisallowedAfter.Contains(InModifierName);
}

bool FActorModifierCoreMetadata::IsDisallowedBefore(const FName& InModifierName) const
{
	return DisallowedBefore.Contains(InModifierName);
}

bool FActorModifierCoreMetadata::IsAllowedAfter(const FName& InModifierName) const
{
	return !IsDisallowedAfter(InModifierName);
}

bool FActorModifierCoreMetadata::IsAllowedBefore(const FName& InModifierName) const
{
	return !IsDisallowedBefore(InModifierName);
}

bool FActorModifierCoreMetadata::IsCompatibleWith(const AActor* InActor) const
{
	if (CompatibilityRuleDelegate.IsBound())
	{
		return CompatibilityRuleDelegate.Execute(InActor);
	}

	return CompatibilityRuleFunction(InActor);
}

bool FActorModifierCoreMetadata::DependsOn(const FName& InModifierName) const
{
	if (InModifierName.IsNone() || Name == InModifierName)
	{
		return false;
	}

	if (Dependencies.Contains(InModifierName))
	{
		return true;
	}

	if (const UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		TArray<FName> OutDependencies;
		return Subsystem->BuildModifierDependencies(Name, OutDependencies) && OutDependencies.Contains(InModifierName);
	}
	return false;
}

bool FActorModifierCoreMetadata::IsRequiredBy(const FName& InModifierName) const
{
	if (InModifierName.IsNone() || Name == InModifierName)
	{
		return false;
	}

	if (const UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		TArray<FName> OutDependencies;
		return Subsystem->BuildModifierDependencies(InModifierName, OutDependencies) && OutDependencies.Contains(Name);
	}
	return false;
}

bool FActorModifierCoreMetadata::ShouldAvoidBefore(FName InCategory) const
{
	return AvoidedBeforeCategories.Contains(InCategory);
}

bool FActorModifierCoreMetadata::ShouldAvoidAfter(FName InCategory) const
{
	return AvoidedAfterCategories.Contains(InCategory);
}

bool FActorModifierCoreMetadata::ResetDefault()
{
	if (const UActorModifierCoreBase* CDO = GetClass()->GetDefaultObject<UActorModifierCoreBase>())
	{
		const FActorModifierCoreMetadata& CDOMetadata = CDO->GetModifierMetadata();

#if WITH_EDITOR
		Color = CDOMetadata.Color;
		Icon = CDOMetadata.Icon;
		bHidden = CDOMetadata.bHidden;
#endif

		Dependencies = CDOMetadata.Dependencies;
		DisallowedAfter = CDOMetadata.DisallowedAfter;
		DisallowedBefore = CDOMetadata.DisallowedBefore;
		bTickAllowed = CDOMetadata.bTickAllowed;
		bMultipleAllowed = CDOMetadata.bMultipleAllowed;
		CompatibilityRuleFunction = CDOMetadata.CompatibilityRuleFunction;
		CompatibilityRuleDelegate = CDOMetadata.CompatibilityRuleDelegate;
		ProfilerFunction = CDOMetadata.ProfilerFunction;

		return true;
	}

	return false;
}

#if WITH_EDITOR
FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetDisplayName(const FText& InName)
{
	DisplayName = InName;

	if (const TSubclassOf<UActorModifierCoreBase> ModifierClass = GetClass())
	{
		if (ModifierClass->IsChildOf<UActorModifierCoreBlueprintBase>())
		{
			DisplayName = FText::Format(LOCTEXT("BlueprintModifierDisplayName", "{0} {1}"), DisplayName, LOCTEXT("BlueprintModifierLabel", "(Blueprint)"));
		}
	}

	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetDescription(const FText& InDescription)
{
	Description = InDescription;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetColor(const FLinearColor& InColor)
{
	Color = InColor;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetIcon(const FSlateIcon& InIcon)
{
	Icon = InIcon;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetHidden(bool bInHidden)
{
	bHidden = bInHidden;
	return *this;
}
#endif

void FActorModifierCoreMetadata::SetupProfilerInstanceInternal(TSharedPtr<FActorModifierCoreProfiler> InProfiler, UActorModifierCoreBase* InModifier, const FName& InProfilerType) const
{
	if (!InProfiler.IsValid() || InProfilerType.IsNone())
	{
		return;
	}

	InProfiler->ConstructInternal(InModifier, InProfilerType);
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetName(FName InName)
{
	Name = InName;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetCategory(FName InCategory)
{
	Category = InCategory;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::AllowTick(bool bInAllowed)
{
	bTickAllowed = bInAllowed;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::AllowMultiple(bool bInAllowed)
{
	bMultipleAllowed = bInAllowed;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::AddDependency(const FName& InModifierName)
{
	Dependencies.AddUnique(InModifierName);
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::DisallowBefore(const FName& InModifierName)
{
	DisallowedBefore.Add(InModifierName);
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::DisallowAfter(const FName& InModifierName)
{
	DisallowedAfter.Add(InModifierName);
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::AvoidBeforeCategory(const FName& InCategory)
{
	AvoidedBeforeCategories.Add(InCategory);
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::AvoidAfterCategory(const FName& InCategory)
{
	AvoidedAfterCategories.Add(InCategory);
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetCompatibilityRule(const TFunction<bool(const AActor*)>& InModifierRule)
{
	CompatibilityRuleFunction = InModifierRule;
	return *this;
}

FActorModifierCoreMetadata& FActorModifierCoreMetadata::SetCompatibilityRule(const FModifierCompatibilityRule& InModifierRule)
{
	CompatibilityRuleDelegate = InModifierRule;
	return *this;
}

UActorModifierCoreBase* FActorModifierCoreMetadata::CreateModifierInstance(UActorModifierCoreStack* InStack) const
{
	if (InStack && InStack->GetModifiedActor())
	{
		UActorModifierCoreBase* NewModifierInstance = NewObject<UActorModifierCoreBase>(InStack, Class, NAME_None, RF_Transactional);
		NewModifierInstance->PostModifierCreation(InStack);
		return NewModifierInstance;
	}

	return nullptr;
}

TSharedPtr<FActorModifierCoreProfiler> FActorModifierCoreMetadata::CreateProfilerInstance(UActorModifierCoreBase* InModifier) const
{
	if (InModifier && InModifier->GetModifiedActor())
	{
		return ProfilerFunction(InModifier);
	}

	return nullptr;
}

const FActorModifierCoreStackSearchOp& FActorModifierCoreStackSearchOp::GetDefault()
{
	static const FActorModifierCoreStackSearchOp Default = {};
	return Default;
}

FActorModifierCoreScopedLock::FActorModifierCoreScopedLock(UActorModifierCoreBase* InModifier)
{
	if (InModifier)
	{
		InModifier->LockModifierExecution();
		ModifiersWeak.Add(InModifier);
	}
}

FActorModifierCoreScopedLock::FActorModifierCoreScopedLock(const TSet<UActorModifierCoreBase*>& InModifiers)
{
	// Locking state to prevent from updating
	for (UActorModifierCoreBase* Modifier : InModifiers)
	{
		if (Modifier)
		{
			Modifier->LockModifierExecution();
            ModifiersWeak.Add(Modifier);
		}
	}
}

FActorModifierCoreScopedLock::~FActorModifierCoreScopedLock()
{
	// Unlocking state of modifier
	for (const TWeakObjectPtr<UActorModifierCoreBase>& ModifierWeak : ModifiersWeak)
	{
		if (UActorModifierCoreBase* Modifier = ModifierWeak.Get())
		{
			Modifier->UnlockModifierExecution();
		}
	}

	ModifiersWeak.Empty();
}

void FActorModifierCoreExecutionTask::Restore()
{
	bool bAllModifiersDirty = false;
	if (ModifierStack)
	{
		bAllModifiersDirty = ModifierStack->bAllModifiersDirty;
	}

	// build the restore chain based on dirty modifiers
	bool bModifierBeforeDirty = false;
	TArray<TObjectPtr<UActorModifierCoreBase>> ModifiersRestoreChain;
	ModifiersRestoreChain.Reserve(Modifiers.Num());
	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (Modifier)
		{
			if ((bAllModifiersDirty || bModifierBeforeDirty || Modifier->IsModifierDirty()))
			{
				bModifierBeforeDirty = true;
				ModifiersRestoreChain.Add(Modifier);
			}
		}
	}

	// unapply modifiers in reverse order
	Algo::Reverse(ModifiersRestoreChain);

	// unapply modifiers
	for (TObjectPtr<UActorModifierCoreBase>& Modifier : ModifiersRestoreChain)
	{
		if (Modifier && Modifier->IsModifierApplied())
		{
			// will only work for modifier that were already executed
			Modifier->Unapply();
		}
	}

	CurrentIndex = INDEX_NONE;
	Modifiers.Empty();
}

void FActorModifierCoreExecutionTask::Apply(TConstArrayView<UActorModifierCoreBase*> InModifiers, UActorModifierCoreStack* InStack)
{
	check(Modifiers.IsEmpty() && CurrentIndex == INDEX_NONE)

	BuildExecutionChain(InModifiers, InStack);

	if (Modifiers.IsValidIndex(CurrentIndex))
	{
		ExecuteCurrentModifier();
	}
	else if (ModifierStack)
	{
		ModifierStack->OnModifierExecutionFinished(/** Result */true);
	}
}

void FActorModifierCoreExecutionTask::Next()
{
	UActorModifierCoreBase* CurrentModifier = Modifiers[CurrentIndex];

	if (!CurrentModifier->IsModifierIdle())
	{
		// unlock current execution state
		CurrentModifier->EndModifierExecution();
	}

	// If it succeeded then unmark modifier dirty and set state as applied (for restore)
	CurrentModifier->bModifierDirty = false;
	CurrentModifier->bModifierApplied = true;

	// Next
	CurrentIndex++;
	ExecuteCurrentModifier();
}

void FActorModifierCoreExecutionTask::Fail()
{
	UActorModifierCoreBase* CurrentModifier = Modifiers[CurrentIndex];

	if (!CurrentModifier->IsModifierIdle())
	{
		// unlock current execution state
		CurrentModifier->EndModifierExecution();
	}
	
	if (ModifierStack)
	{
		ModifierStack->OnModifierExecutionFinished(/** Result */false);
	}
}

void FActorModifierCoreExecutionTask::Skip()
{
	UActorModifierCoreBase* CurrentModifier = Modifiers[CurrentIndex];

	// Skipped
	CurrentModifier->Status = FActorModifierCoreStatus(EActorModifierCoreStatus::Success, FText::GetEmpty());
	CurrentModifier->bModifierDirty = false;
	CurrentModifier->bModifierApplied = false;

	// Next
	CurrentIndex++;
	ExecuteCurrentModifier();
}

UActorModifierCoreBase* FActorModifierCoreExecutionTask::GetCurrentModifier() const
{
	if (Modifiers.IsValidIndex(CurrentIndex))
	{
		return Modifiers[CurrentIndex];
	}

	return nullptr;
}

void FActorModifierCoreExecutionTask::ReplaceModifier(UActorModifierCoreBase& InOldModifier, UActorModifierCoreBase& InNewModifier)
{
	const int32 Index = Modifiers.Find(&InOldModifier);
	
	if (Modifiers.IsValidIndex(Index))
	{
		Modifiers[Index] = &InNewModifier;
	}
}

void FActorModifierCoreExecutionTask::RemoveModifier(UActorModifierCoreBase& InModifier)
{
	Modifiers.Remove(&InModifier);
}

void FActorModifierCoreExecutionTask::BuildExecutionChain(TConstArrayView<UActorModifierCoreBase*> InModifiers, UActorModifierCoreStack* InStack)
{
	ModifierStack = InStack;
	Modifiers = InModifiers;

	bool bAllModifiersDirty = false;

	if (ModifierStack)
	{
		if (!ModifierStack->IsModifierEnabled())
		{
			return;
		}

		bAllModifiersDirty = ModifierStack->bAllModifiersDirty;
	}

	for (int32 Index = 0; Index < Modifiers.Num(); ++Index)
	{
		if (UActorModifierCoreBase* Modifier = Modifiers[Index])
		{
			if (bAllModifiersDirty
				|| Modifier->IsModifierDirty()
				// Give a chance to non tickable modifier to set themselves dirty too
				|| (!Modifier->GetModifierMetadata().IsTickAllowed() && Modifier->IsModifierDirtyable()))
			{
				CurrentIndex = Index;
				break;
			}
		}
	}
}

void FActorModifierCoreExecutionTask::ExecuteCurrentModifier()
{
	// are we done with this execution round
	if (CurrentIndex > Modifiers.Num() - 1)
	{
		if (ModifierStack)
		{
			ModifierStack->OnModifierExecutionFinished(/** Result */true);
		}

		return;
	}

	const TObjectPtr<UActorModifierCoreBase>& Modifier = Modifiers[CurrentIndex];

	if (!IsValid(Modifier))
	{
		if (ModifierStack)
		{
			ModifierStack->OnModifierExecutionFinished(/** Result */false);
		}

		return;
	}

	const bool bValidActor = IsValid(Modifier->GetModifiedActor());
	const bool bModifierReady = Modifier->bModifierIdle && Modifier->IsModifierReady();

	if (bValidActor && bModifierReady)
	{
		const bool bModifierStackFrozen = ModifierStack && ModifierStack->IsModifierStackFrozen();
		if (!bModifierStackFrozen && (Modifier->bModifierEnabled || Modifier->IsModifierStack()))
		{
			// lock current execution state
			Modifier->BeginModifierExecution();

			// save the state before this modifier is execute to restore it later
			Modifier->SavePreState();

			// run modifier logic, and update the modifier dirty state once the modifier logic completes
			Modifier->Apply();
		}
		// disable
		else
		{
			Skip();
		}
	}
	// invalid
	else
	{
		Fail();
	}
}

#undef LOCTEXT_NAMESPACE
