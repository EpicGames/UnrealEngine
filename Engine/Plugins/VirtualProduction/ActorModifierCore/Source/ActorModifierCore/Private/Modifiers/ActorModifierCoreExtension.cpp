// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreExtension.h"
#include "Modifiers/ActorModifierCoreBase.h"

FActorModifierCoreExtension::~FActorModifierCoreExtension()
{
	DisableExtension(EActorModifierCoreDisableReason::Destroyed);
}

void FActorModifierCoreExtension::ConstructInternal(UActorModifierCoreBase* InModifier, const FName& InExtensionType)
{
	if (bExtensionInitialized)
	{
		return;
	}

	ModifierWeak = InModifier;
	ExtensionType = InExtensionType;
	bExtensionInitialized = true;
	OnExtensionInitialized();
}

void FActorModifierCoreExtension::EnableExtension(EActorModifierCoreEnableReason InReason)
{
	const UActorModifierCoreBase* Modifier = GetModifier();

	if (!bExtensionEnabled && Modifier)
	{
		bExtensionEnabled = true;
		OnExtensionEnabled(InReason);
	}
}

void FActorModifierCoreExtension::DisableExtension(EActorModifierCoreDisableReason InReason)
{
	if (bExtensionEnabled)
	{
		bExtensionEnabled = false;
		OnExtensionDisabled(InReason);
	}
}

AActor* FActorModifierCoreExtension::GetModifierActor() const
{
	const UActorModifierCoreBase* Modifier = GetModifier();
	return Modifier ? Modifier->GetModifiedActor() : nullptr;
}

UWorld* FActorModifierCoreExtension::GetModifierWorld() const
{
	const AActor* ModifiedActor = GetModifierActor();
	return ModifiedActor ? ModifiedActor->GetWorld() : nullptr;
}

ULevel* FActorModifierCoreExtension::GetModifierLevel() const
{
	const AActor* ModifiedActor = GetModifierActor();
	return ModifiedActor ? ModifiedActor->GetLevel() : nullptr;
}
