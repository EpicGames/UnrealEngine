// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreSharedActor.h"

#include "Subsystems/ActorModifierCoreSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreSharedActor, Log, All);

AActorModifierCoreSharedActor::AActorModifierCoreSharedActor()
{
	OnDestroyed.AddUniqueDynamic(this, &AActorModifierCoreSharedActor::OnActorDestroyed);
}

void AActorModifierCoreSharedActor::PostLoad()
{
	Super::PostLoad();

	if (const UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		Subsystem->RegisterModifierSharedProvider(this);
	}

	// Remove invalid entries (transient ones)
	for (TMap<TObjectPtr<UClass>, TObjectPtr<UActorModifierCoreSharedObject>>::TIterator It(SharedObjects); It; ++It)
	{
		if (!IsValid(It->Value.Get()))
		{
			It.RemoveCurrent();
			continue;
		}

		UE_LOGF(LogActorModifierCoreSharedActor, Verbose, "%ls : Modifier Shared object of class %ls loaded", *GetActorNameOrLabel(), *It->Key->GetName());
	}
}

UActorModifierCoreSharedObject* AActorModifierCoreSharedActor::FindOrAddShared(TSubclassOf<UActorModifierCoreSharedObject> InSharedClass)
{
	if (!InSharedClass.Get())
	{
		return nullptr;
	}

	if (UActorModifierCoreSharedObject* SharedObject = FindShared(InSharedClass))
	{
		return SharedObject;
	}

	UActorModifierCoreSharedObject* NewSharedObject = NewObject<UActorModifierCoreSharedObject>(this, InSharedClass);
	SharedObjects.Add(InSharedClass, NewSharedObject);

	UE_LOGF(LogActorModifierCoreSharedActor, Verbose, "%ls : Modifier Shared object of class %ls created", *GetActorNameOrLabel(), *InSharedClass->GetName());

	return NewSharedObject;
}

UActorModifierCoreSharedObject* AActorModifierCoreSharedActor::FindShared(TSubclassOf<UActorModifierCoreSharedObject> InSharedClass) const
{
	if (const TObjectPtr<UActorModifierCoreSharedObject>* SharedObject = SharedObjects.Find(InSharedClass))
	{
		if (IsValid(SharedObject->Get()))
		{
			return SharedObject->Get();
		}
	}

	return nullptr;
}

void AActorModifierCoreSharedActor::OnActorDestroyed(AActor* InActor)
{
	if (const UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		Subsystem->UnregisterModifierSharedProvider(InActor);
	}
}
