// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "ActorModifierCoreSharedActor.generated.h"

/**
	Singleton actor used to keep all shared data across modifiers instances,
	there can only be one per level, should not be visible to user
	Use the modifier subsystem to query objects, do not handle it directly
*/
UCLASS(NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType)
class AActorModifierCoreSharedActor : public AActor
{
	friend class UActorModifierCoreSubsystem;

	GENERATED_BODY()

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	AActorModifierCoreSharedActor();

	/** Queries a shared object or create one of a specific class if none */
	template<typename InSharedClass = UActorModifierCoreSharedObject, typename = typename TEnableIf<TIsDerivedFrom<InSharedClass, UActorModifierCoreSharedObject>::Value>::Type>
	InSharedClass* FindOrAddShared()
	{
		UClass* SharedClass = InSharedClass::StaticClass();
		return Cast<InSharedClass>(FindOrAddShared(SharedClass));
	}

	/** Queries a shared object of a specific class */
	template<typename InSharedClass = UActorModifierCoreSharedObject, typename = typename TEnableIf<TIsDerivedFrom<InSharedClass, UActorModifierCoreSharedObject>::Value>::Type>
	InSharedClass* FindShared() const
	{
		UClass* SharedClass = InSharedClass::StaticClass();
		return Cast<InSharedClass>(FindShared(SharedClass));
	}

	/** Queries a shared object or create one if none */
	UActorModifierCoreSharedObject* FindOrAddShared(TSubclassOf<UActorModifierCoreSharedObject> InSharedClass);

	/** Queries a shared object */
	UActorModifierCoreSharedObject* FindShared(TSubclassOf<UActorModifierCoreSharedObject> InSharedClass) const;

	/** Called when actor is destroyed */
	UFUNCTION()
	void OnActorDestroyed(AActor* InActor);

	/** Map of Shared objects available */
	UPROPERTY()
	TMap<TObjectPtr<UClass>, TObjectPtr<UActorModifierCoreSharedObject>> SharedObjects;
};
