// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreStack.h"

#include "ActorModifierCoreLog.h"
#include "Algo/RemoveIf.h"
#include "Components/SceneComponent.h"
#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreStack"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreStack, Log, All);

UActorModifierCoreStack::FOnModifierAdded UActorModifierCoreStack::OnModifierAddedDelegate;
UActorModifierCoreStack::FOnModifierRemoved UActorModifierCoreStack::OnModifierRemovedDelegate;
UActorModifierCoreStack::FOnModifierUpdated UActorModifierCoreStack::OnModifierMovedDelegate;
UActorModifierCoreStack::FOnModifierUpdated UActorModifierCoreStack::OnModifierReplacedDelegate;

UActorModifierCoreStack* UActorModifierCoreStack::Create(UActorModifierCoreComponent* InComponent, UActorModifierCoreStack* InParentStack)
{
	if (IsValid(InComponent) && (!InParentStack || InParentStack->GetModifiedActor() == InComponent->GetOwner()))
	{
		UObject* NewOuter = nullptr;
		if (InParentStack)
		{
			NewOuter = InParentStack;
		}
		else
		{
			NewOuter = InComponent;
		}
		UActorModifierCoreStack* NewStack = NewObject<UActorModifierCoreStack>(NewOuter, NAME_None, RF_Transactional);
		NewStack->PostModifierCreation(InParentStack);
		NewStack->InitializeModifier(EActorModifierCoreEnableReason::User);
		return NewStack;
	}
	return nullptr;
}

bool UActorModifierCoreStack::IsModifierStackFrozen() const
{
	return bModifierStackFrozen;
}

void UActorModifierCoreStack::SetModifierStackFrozen(bool bInFrozen)
{
	if (bModifierStackFrozen == bInFrozen)
	{
		return;
	}

	bModifierStackFrozen = bInFrozen;
	OnModifierStackFrozenChanged();
}

bool UActorModifierCoreStack::IsModifierReady() const
{
	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (!Modifier
			|| !Modifier->IsModifierReady()
			|| !Modifier->IsModifierIdle()
			|| Modifier->IsModifierExecutionLocked())
		{
			return false;
		}
	}
	return true;
}

void UActorModifierCoreStack::ProcessFunctionOnRestore(const TFunction<void()>& InFunction)
{
	OnRestoreFunctions.Add(InFunction);
}

void UActorModifierCoreStack::ProcessFunctionOnIdle(const TFunction<void()>& InFunction)
{
	if (IsModifierIdle() && !IsModifierExecutionLocked())
	{
		InFunction();
	}
	else
	{
		OnIdleFunctions.Add(InFunction);
	}
}

bool UActorModifierCoreStack::IsModifierStackInitialized() const
{
	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (Modifier && !Modifier->IsModifierInitialized())
		{
			return false;
		}
	}
	return true;
}

void UActorModifierCoreStack::SetModifierProfiling(bool bInProfiling)
{
	bModifierProfiling = bInProfiling;

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Profiling %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, bInProfiling ? TEXT("enabled") : TEXT("disabled"));
}

void UActorModifierCoreStack::PostLoad()
{
	// remove invalid
	Modifiers.SetNum(Algo::RemoveIf(Modifiers, [](const UActorModifierCoreBase* InModifier)
	{
		return !InModifier;
	}));

	Super::PostLoad();
}

#if WITH_EDITOR
void UActorModifierCoreStack::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierStackFrozen))
	{
		OnModifierStackFrozenChanged();
	}
}
#endif

void UActorModifierCoreStack::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	if (UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		// initialize only root stack
		if (IsRootStack())
		{
			// register with subsystem for fast query
			if (ModifierSubsystem->RegisterActorModifierStack(this))
			{
				// initialize events when actor changes
				if (AActor* Actor = GetModifiedActor())
				{
					if (USceneComponent* RootComponent = ModifiedActor->GetRootComponent())
					{
						if (!RootComponent->TransformUpdated.IsBoundToObject(this))
						{
							RootComponent->TransformUpdated.AddUObject(this, &UActorModifierCoreStack::OnActorTransformUpdated);
						}
					}
				}
			}
		}

		ModifierSubsystem->OnModifierReplaced().AddUObject(this, &UActorModifierCoreStack::OnBlueprintModifierReplaced);
	}

	// initialize inner modifiers if they are not initialized
	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (Modifier)
		{
			Modifier->InitializeModifier(InReason);
		}
	}
}

bool UActorModifierCoreStack::ProcessFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	if (!InSearchOptions.bSkipStack)
	{
		if (!InFunction(this))
		{
			return false;
		}
	}

	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (Modifier)
		{
			if (Modifier->IsModifierStack() && !InSearchOptions.bRecurse)
			{
				if (!InSearchOptions.bSkipStack)
				{
					if (!InFunction(this))
					{
						return false;
					}
				}
			}
			else
			{
				if (!Modifier->ProcessFunction(InFunction, InSearchOptions))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool UActorModifierCoreStack::ProcessSearchFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	bool bResult = true;

	if (InSearchOptions.Position == EActorModifierCoreStackPosition::After)
	{
		bool bAfter = !InSearchOptions.PositionContext;
		bResult = ProcessFunction([&InSearchOptions, &bAfter, &InFunction](const UActorModifierCoreBase* InModifier)->bool
		{
			if (InModifier == InSearchOptions.PositionContext)
			{
				bAfter = true;
				return true;
			}

			if (bAfter && !InFunction(InModifier))
			{
				return false;
			}

			return true;
		}, InSearchOptions);
	}
	else if (InSearchOptions.Position == EActorModifierCoreStackPosition::Before)
	{
		bResult = ProcessFunction([&InSearchOptions, &InFunction](const UActorModifierCoreBase* InModifier)->bool
		{
			if (InModifier == InSearchOptions.PositionContext)
			{
				return false;
			}

			if (!InFunction(InModifier))
			{
				return false;
			}

			return true;
		}, InSearchOptions);
	}

	return bResult;
}

UActorModifierCoreBase* UActorModifierCoreStack::GetFirstModifier() const
{
	if (Modifiers.Num() >= 1)
	{
		return Modifiers[0];
	}
	return nullptr;
}

UActorModifierCoreBase* UActorModifierCoreStack::GetLastModifier() const
{
	if (Modifiers.Num() >= 1)
	{
		return Modifiers.Last();
	}
	return nullptr;
}

bool UActorModifierCoreStack::ContainsModifier(const FName& InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	return FindModifier(InSearchName, InSearchOptions) != nullptr;
}

bool UActorModifierCoreStack::ContainsModifier(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	return FindModifier(InSearchClass, InSearchOptions) != nullptr;
}

bool UActorModifierCoreStack::ContainsModifier(const UActorModifierCoreBase* InSearchModifier, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	if (!InSearchModifier)
	{
		return false;
	}

	bool bFound = false;
	ProcessSearchFunction([InSearchModifier, &bFound](const UActorModifierCoreBase* InModifier)->bool
	{
		// false to stop processing
		if (InModifier && InModifier == InSearchModifier)
		{
			bFound = true;
			return false;
		}

		return true;
	}, InSearchOptions);

	return bFound;
}

UActorModifierCoreBase* UActorModifierCoreStack::FindModifier(FName InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	UActorModifierCoreBase* FoundModifier = nullptr;

	if (InSearchName.IsNone())
	{
		return FoundModifier;
	}

	ProcessSearchFunction([InSearchName, &FoundModifier](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier && InModifier->GetModifierName() == InSearchName)
		{
			FoundModifier = const_cast<UActorModifierCoreBase*>(InModifier);
			return false;
		}

		return true;
	}, InSearchOptions);

	return FoundModifier;
}

UActorModifierCoreBase* UActorModifierCoreStack::FindModifier(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	UActorModifierCoreBase* FoundModifier = nullptr;

	if (!InSearchClass || !InSearchClass->IsChildOf<UActorModifierCoreBase>())
	{
		return FoundModifier;
	}

	ProcessSearchFunction([InSearchClass, &FoundModifier](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier && InModifier->IsA(InSearchClass))
		{
			FoundModifier = const_cast<UActorModifierCoreBase*>(InModifier);
			return false;
		}

		return true;
	}, InSearchOptions);

	return FoundModifier;
}

TArray<UActorModifierCoreBase*> UActorModifierCoreStack::FindModifiers(FName InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	TArray<UActorModifierCoreBase*> FoundModifiers;

	if (InSearchName.IsNone())
	{
		return FoundModifiers;
	}

	ProcessSearchFunction([InSearchName, &FoundModifiers](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier && InModifier->GetModifierName() == InSearchName)
		{
			FoundModifiers.Add(const_cast<UActorModifierCoreBase*>(InModifier));
		}

		return true;
	}, InSearchOptions);

	return FoundModifiers;
}

TArray<UActorModifierCoreBase*> UActorModifierCoreStack::FindModifiers(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions) const
{
	TArray<UActorModifierCoreBase*> FoundModifiers;

	if (!InSearchClass || !InSearchClass->IsChildOf<UActorModifierCoreBase>())
	{
		return FoundModifiers;
	}

	ProcessSearchFunction([InSearchClass, &FoundModifiers](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier && InModifier->IsA(InSearchClass))
		{
			FoundModifiers.Add(const_cast<UActorModifierCoreBase*>(InModifier));
		}

		return true;
	}, InSearchOptions);

	return FoundModifiers;
}

bool UActorModifierCoreStack::IsRootStack() const
{
	return !GetModifierStack();
}

bool UActorModifierCoreStack::ContainsModifierBefore(const FName& InSearchName, const UActorModifierCoreBase* InBeforeModifier) const
{
	if (!IsValid(InBeforeModifier))
	{
		return ContainsModifier(InSearchName);
	}

	bool bFound = false;
	ProcessFunction([InSearchName, InBeforeModifier, &bFound](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier == InBeforeModifier)
		{
			return false;
		}

		if (InModifier && InModifier->GetModifierName() == InSearchName)
		{
			bFound = true;
			return false;
		}

		return true;
	});

	return bFound;
}

bool UActorModifierCoreStack::ContainsModifierBefore(const UActorModifierCoreBase* InSearchModifier, const UActorModifierCoreBase* InBeforeModifier) const
{
	if (!IsValid(InBeforeModifier))
	{
		return ContainsModifier(InSearchModifier);
	}

	bool bFound = false;
	ProcessFunction([InSearchModifier, InBeforeModifier, &bFound](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier == InBeforeModifier)
		{
			return false;
		}

		if (InModifier && InModifier == InSearchModifier)
		{
			bFound = true;
			return false;
		}

		return true;
	});

	return bFound;
}

bool UActorModifierCoreStack::ContainsModifierAfter(const FName& InSearchName, const UActorModifierCoreBase* InAfterModifier) const
{
	if (!IsValid(InAfterModifier))
	{
		return ContainsModifier(InSearchName);
	}

	bool bFound = false;
	bool bAfter = false;
	ProcessFunction([InSearchName, InAfterModifier, &bFound, &bAfter](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier == InAfterModifier)
		{
			bAfter = true;
			return true;
		}

		if (bAfter && InModifier && InModifier->GetModifierName() == InSearchName)
		{
			bFound = true;
			return false;
		}

		return true;
	});

	return bFound;
}

bool UActorModifierCoreStack::ContainsModifierAfter(const UActorModifierCoreBase* InSearchModifier, const UActorModifierCoreBase* InAfterModifier) const
{
	if (!IsValid(InAfterModifier))
	{
		return ContainsModifier(InSearchModifier);
	}

	bool bFound = false;
	bool bAfter = false;
	ProcessFunction([InSearchModifier, InAfterModifier, &bFound, &bAfter](const UActorModifierCoreBase* InModifier)->bool
	{
		if (InModifier == InAfterModifier)
		{
			bAfter = true;
			return true;
		}

		if (bAfter && InModifier && InModifier == InSearchModifier)
		{
			bFound = true;
			return false;
		}

		return true;
	});

	return bFound;
}

UActorModifierCoreBase* UActorModifierCoreStack::CloneModifier(FActorModifierCoreStackCloneOp& InCloneOp)
{
	if (IsModifierStackFrozen())
	{
		if (InCloneOp.FailReason)
		{
			*InCloneOp.FailReason = LOCTEXT("CloneFailed_FrozenStack", "Modifier Stack is Frozen");
		}
		return nullptr;
	}

	UActorModifierCoreBase* NewInsertedModifier = nullptr;

	if (!IsValid(InCloneOp.CloneModifier) || InCloneOp.CloneModifier->GetModifiedActor() == GetModifiedActor())
	{
		return NewInsertedModifier;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierSubsystem)
	{
		return NewInsertedModifier;
	}

	// if we insert it after input modifier, lets find the next modifier to insert before and reuse the same logic
	// insert before is easier than insert after because we don't need to handle new modifiers added as dependencies
	UActorModifierCoreBase* InsertBeforeModifier = nullptr;

	// Where do we insert this new modifier
	int32 InsertIdx = INDEX_NONE;

	if (InCloneOp.ClonePosition == EActorModifierCoreStackPosition::Before)
	{
		// when inserting before, when modifier is null => insert at last position
		if (!IsValid(InCloneOp.ClonePositionContext))
		{
			InsertIdx = Modifiers.Num();
			InsertBeforeModifier = nullptr;
		}
		else
		{
			InsertIdx = Modifiers.Find(InCloneOp.ClonePositionContext);
			InsertBeforeModifier = InCloneOp.ClonePositionContext;
		}
	}
	else if (InCloneOp.ClonePosition == EActorModifierCoreStackPosition::After)
	{
		// when inserting after, when modifier is null => insert at position 0
		if (!IsValid(InCloneOp.ClonePositionContext))
		{
			InsertIdx = 0;
			InsertBeforeModifier = Modifiers.IsValidIndex(0) ? Modifiers[0] : nullptr;
		}
		else
		{
			InsertIdx = Modifiers.Find(InCloneOp.ClonePositionContext);
			if (InsertIdx != INDEX_NONE)
			{
				InsertIdx += 1;
				InsertBeforeModifier = Modifiers.IsValidIndex(InsertIdx) ? Modifiers[InsertIdx] : nullptr;
			}
		}
	}

	if (InsertIdx == INDEX_NONE)
	{
		if (InCloneOp.FailReason)
		{
			*InCloneOp.FailReason = FText::Format(
				LOCTEXT("CloneModifierNotFound", "Could not clone modifier {0} in this stack"),
				FText::FromName(InCloneOp.CloneModifier->GetModifierName()));
		}

		return NewInsertedModifier;
	}

	// Get dependencies of the new modifier we want to add
	TArray<FName> OutRequiredModifiers;
	if (!ModifierSubsystem->BuildModifierDependencies(InCloneOp.CloneModifier->GetModifierName(), OutRequiredModifiers) || OutRequiredModifiers.IsEmpty())
	{
		if (InCloneOp.FailReason)
		{
			*InCloneOp.FailReason = FText::Format(
				LOCTEXT("InvalidModifierDependencies", "Could not build modifier dependencies for modifier {0}"),
				FText::FromName(InCloneOp.CloneModifier->GetModifierName()));
		}

		return NewInsertedModifier;
	}

	// Get modifiers allowed by this stack and before the insert modifier
	const TSet<FName> AllowedModifiers = ModifierSubsystem->GetAllowedModifiers(GetModifiedActor(), InsertBeforeModifier, EActorModifierCoreStackPosition::Before);

	// Do we support the input modifier
	if (!AllowedModifiers.Contains(InCloneOp.CloneModifier->GetModifierName()))
	{
		if (InCloneOp.FailReason)
		{
			*InCloneOp.FailReason = FText::Format(
				LOCTEXT("UnsupportedModifier", "Modifier {0} is not supported by this stack"),
				FText::FromName(InCloneOp.CloneModifier->GetModifierName()));
		}

		return NewInsertedModifier;
	}

	// Check that we have or can add every dependency needed
	for (int32 Idx = OutRequiredModifiers.Num() - 2; Idx >= 0; Idx--)
	{
		const FName& RequiredModifier = OutRequiredModifiers[Idx];

		// Do we have it already in the stack
		if (ContainsModifierBefore(RequiredModifier, InsertBeforeModifier))
		{
			OutRequiredModifiers.RemoveAt(Idx);
			continue;
		}

		// Do we support this required modifier
		if (AllowedModifiers.Contains(RequiredModifier))
		{
			continue;
		}

		if (InCloneOp.FailReason)
		{
			*InCloneOp.FailReason = FText::Format(
				LOCTEXT("UnsupportedModifierDependencies", "Required modifier {0} is not supported by this stack"),
				FText::FromName(RequiredModifier));
		}

		return NewInsertedModifier;
	}

#if WITH_EDITOR
	Modify();
#endif

	// add requested modifier with its dependencies before
	for (const FName& RequireModifier : OutRequiredModifiers)
	{
		// try to create new required modifier
		FText FailReason;
		UActorModifierCoreBase* NewModifier = nullptr;
		if (RequireModifier == InCloneOp.CloneModifier->GetModifierName())
		{
			FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(InCloneOp.CloneModifier, this);
			NewModifier = Cast<UActorModifierCoreBase>(StaticDuplicateObjectEx(Parameters));
			NewModifier->ModifiedActor = GetModifiedActor();
		}
		else
		{
			NewModifier = ModifierSubsystem->CreateModifierInstance(RequireModifier, this, FailReason, InsertBeforeModifier);
		}

		if (NewModifier)
		{
			Modifiers.Insert(NewModifier, InsertIdx);

			// added successfully
			NewModifier->InitializeModifier(EActorModifierCoreEnableReason::User);

			// to be able to receive postEditUndo events
#if WITH_EDITOR
			NewModifier->Modify();
#endif

			InsertIdx++;

			if (NewModifier->GetModifierName() == InCloneOp.CloneModifier->GetModifierName())
			{
				NewInsertedModifier = NewModifier;
			}
			else if (InCloneOp.AddedDependencies)
			{
				InCloneOp.AddedDependencies->Add(NewModifier);
			}
		}
		else
		{
			if (InCloneOp.FailReason)
			{
				*InCloneOp.FailReason = FailReason;
			}
			return nullptr;
		}
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Clone modifier %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *InCloneOp.CloneModifier->GetModifierName().ToString());

	// Refresh the stack
	MarkModifierDirty();

	// Check stack optimization
	ScheduleModifierOptimization(true);

	return NewInsertedModifier;
}

UActorModifierCoreBase* UActorModifierCoreStack::InsertModifier(FActorModifierCoreStackInsertOp& InInsertOp)
{
	if (IsModifierStackFrozen())
	{
		if (InInsertOp.FailReason)
		{
			*InInsertOp.FailReason = LOCTEXT("InsertFailed_FrozenStack", "Modifier Stack is Frozen");
		}
		return nullptr;
	}

	UActorModifierCoreBase* NewInsertedModifier = nullptr;

	if (InInsertOp.NewModifierName.IsNone())
	{
		return NewInsertedModifier;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierSubsystem)
	{
		return NewInsertedModifier;
	}

	// if we insert it after input modifier, lets find the next modifier to insert before and reuse the same logic
	// insert before is easier than insert after because we don't need to handle new modifiers added as dependencies
	UActorModifierCoreBase* InsertBeforeModifier = nullptr;

	// Where do we insert this new modifier
	int32 InsertIdx = INDEX_NONE;

	if (InInsertOp.InsertPosition == EActorModifierCoreStackPosition::Before)
	{
		// when inserting before, when modifier is null => insert at last position
		if (!IsValid(InInsertOp.InsertPositionContext))
		{
			InsertIdx = Modifiers.Num();
			InsertBeforeModifier = nullptr;
		}
		else
		{
			InsertIdx = Modifiers.Find(InInsertOp.InsertPositionContext);
			InsertBeforeModifier = InInsertOp.InsertPositionContext;
		}
	}
	else if (InInsertOp.InsertPosition == EActorModifierCoreStackPosition::After)
	{
		// when inserting after, when modifier is null => insert at position 0
		if (!IsValid(InInsertOp.InsertPositionContext))
		{
			InsertIdx = 0;
			InsertBeforeModifier = Modifiers.IsValidIndex(0) ? Modifiers[0] : nullptr;
		}
		else
		{
			InsertIdx = Modifiers.Find(InInsertOp.InsertPositionContext);
			if (InsertIdx != INDEX_NONE)
			{
				InsertIdx += 1;
				InsertBeforeModifier = Modifiers.IsValidIndex(InsertIdx) ? Modifiers[InsertIdx] : nullptr;
			}
		}
	}

	if (InsertIdx == INDEX_NONE)
	{
		if (InInsertOp.FailReason)
		{
			*InInsertOp.FailReason = FText::Format(
				LOCTEXT("InsertModifierNotFound", "Could not insert modifier {0} in this stack"),
				FText::FromName(InInsertOp.NewModifierName));
		}

		return NewInsertedModifier;
	}

	// Get dependencies of the new modifier we want to add, should always be at least 1 for itself
	TArray<FName> OutRequiredModifiers;
	if (!ModifierSubsystem->BuildModifierDependencies(InInsertOp.NewModifierName, OutRequiredModifiers) || OutRequiredModifiers.IsEmpty())
	{
		if (InInsertOp.FailReason)
		{
			*InInsertOp.FailReason = FText::Format(
				LOCTEXT("InvalidModifierDependencies", "Could not build modifier dependencies for modifier {0}"),
				FText::FromName(InInsertOp.NewModifierName));
		}

		return NewInsertedModifier;
	}

	// Get modifiers allowed by this stack and before the insert context modifier
	const TSet<FName> AllowedModifiers = ModifierSubsystem->GetAllowedModifiers(GetModifiedActor(), InsertBeforeModifier, EActorModifierCoreStackPosition::Before);

	// Do we support the input modifier
	if (!AllowedModifiers.Contains(InInsertOp.NewModifierName))
	{
		if (InInsertOp.FailReason)
		{
			*InInsertOp.FailReason = FText::Format(
				LOCTEXT("UnsupportedModifier", "Modifier {0} is not supported by this stack"),
				FText::FromName(InInsertOp.NewModifierName));
		}

		return NewInsertedModifier;
	}

	// Check that we have or can add every dependency needed
	for (int32 Idx = OutRequiredModifiers.Num() - 2; Idx >= 0; Idx--)
	{
		const FName& RequiredModifier = OutRequiredModifiers[Idx];

		// Do we have it already in the stack
		if (ContainsModifierBefore(RequiredModifier, InsertBeforeModifier))
		{
			OutRequiredModifiers.RemoveAt(Idx);
			continue;
		}

		// Do we support this required modifier
		if (AllowedModifiers.Contains(RequiredModifier))
		{
			continue;
		}

		if (InInsertOp.FailReason)
		{
			*InInsertOp.FailReason = FText::Format(
				LOCTEXT("UnsupportedModifierDependencies", "Required modifier {0} is not supported by this stack"),
				FText::FromName(RequiredModifier));
		}

		return NewInsertedModifier;
	}

#if WITH_EDITOR
	Modify();
#endif

	// add requested modifier with its dependencies before
	for (const FName& RequireModifier : OutRequiredModifiers)
	{
		// try to create new required modifier
		FText FailReason;
		if (UActorModifierCoreBase* NewModifier = ModifierSubsystem->CreateModifierInstance(RequireModifier, this, FailReason, InsertBeforeModifier))
		{
			Modifiers.Insert(NewModifier, InsertIdx);

			// added successfully
			NewModifier->InitializeModifier(EActorModifierCoreEnableReason::User);

			// to be able to receive postEditUndo events
#if WITH_EDITOR
			NewModifier->Modify();
#endif

			InsertIdx++;

			if (NewModifier->GetModifierName() == InInsertOp.NewModifierName)
			{
				NewInsertedModifier = NewModifier;
			}
			else if (InInsertOp.AddedDependencies)
			{
				InInsertOp.AddedDependencies->Add(NewModifier);
			}
		}
		else
		{
			if (InInsertOp.FailReason)
			{
				*InInsertOp.FailReason = FailReason;
			}
			return nullptr;
		}
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Insert modifier %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *InInsertOp.NewModifierName.ToString());

	// Refresh the stack
	MarkModifierDirty();

	// Check stack optimization
	ScheduleModifierOptimization(true);

	return NewInsertedModifier;
}

bool UActorModifierCoreStack::MoveModifier(FActorModifierCoreStackMoveOp& InMoveOp)
{
	if (IsModifierStackFrozen())
	{
		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = LOCTEXT("MoveFailed_FrozenStack", "Modifier Stack is Frozen");
		}
		return false;
	}

	if (!IsValid(InMoveOp.MoveModifier))
	{
		return false;
	}

	// cannot move after or before itself
	if (InMoveOp.MoveModifier == InMoveOp.MovePositionContext)
	{
		return false;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierSubsystem)
	{
		return false;
	}

	const int32 CurrentIdx = Modifiers.Find(InMoveOp.MoveModifier);

	if (CurrentIdx == INDEX_NONE)
	{
		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = FText::Format(
			LOCTEXT("MoveSourceModifierNotFound", "Could not move modifier {0} in this stack"),
				FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
		}

		return false;
	}

	// if we move it after input modifier, lets find the next modifier to move before and reuse the same logic
	UActorModifierCoreBase* MoveBeforeModifier = nullptr;

	// Where do we move this new modifier
	int32 MoveIdx = INDEX_NONE;

	if (InMoveOp.MovePosition == EActorModifierCoreStackPosition::Before)
	{
		// when moving before, when modifier is null => move at last position
		if (!IsValid(InMoveOp.MovePositionContext))
		{
			MoveIdx = Modifiers.Num();
			MoveBeforeModifier = nullptr;
		}
		else
		{
			MoveIdx = Modifiers.Find(InMoveOp.MovePositionContext);
			MoveBeforeModifier = InMoveOp.MovePositionContext;
		}
	}
	else if (InMoveOp.MovePosition == EActorModifierCoreStackPosition::After)
	{
		// when moving after, when modifier is null => move at position 0
		if (!IsValid(InMoveOp.MovePositionContext))
		{
			MoveIdx = 0;
			MoveBeforeModifier = Modifiers.IsValidIndex(0) ? Modifiers[0] : nullptr;
		}
		else
		{
			MoveIdx = Modifiers.Find(InMoveOp.MovePositionContext);
			if (MoveIdx != INDEX_NONE)
			{
				MoveIdx += 1;
				MoveBeforeModifier = Modifiers.IsValidIndex(MoveIdx) ? Modifiers[MoveIdx] : nullptr;
			}
		}
	}

	if (MoveIdx == INDEX_NONE)
	{
		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = FText::Format(
				LOCTEXT("MoveTargetModifierNotFound", "Could not move modifier {0} in this stack"),
				FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
		}

		return false;
	}

	if (MoveIdx == CurrentIdx)
	{
		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = FText::Format(
				LOCTEXT("MoveModifierInvalid", "Modifier {0} is already at this position in this stack"),
				FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
		}

		return false;
	}

	// Get modifiers required by this modifier, eg: will return subdivide for bend
	TSet<UActorModifierCoreBase*> OutRequiredModifiers;
	GetRequiredModifiers(InMoveOp.MoveModifier, OutRequiredModifiers);

	// Gets modifiers depending on this modifier, eg: will return bend for subdivide
	TSet<UActorModifierCoreBase*> OutDependentModifiers;
	GetDependentModifiers(InMoveOp.MoveModifier, OutDependentModifiers);

	// allow move if required modifiers stay before & dependent modifiers stay after
	bool bCurrentModifierBefore = true;
	const bool bMoveAllowed = ProcessFunction([MoveBeforeModifier, &bCurrentModifierBefore, &OutRequiredModifiers, &OutDependentModifiers, &InMoveOp, ModifierSubsystem](const UActorModifierCoreBase* InModifier)->bool
	{
		if (MoveBeforeModifier && InModifier == MoveBeforeModifier)
		{
			bCurrentModifierBefore = false;
		}

		if (InModifier == InMoveOp.MoveModifier)
		{
			return true;
		}

		// is the current modifier before the MoveBeforeModifier
		if (bCurrentModifierBefore)
		{
			// do we allow current modifier to be before the move modifier
			bool bAllowedBefore = ModifierSubsystem->ProcessModifierMetadata(InModifier->GetModifierName(), [&InMoveOp](const FActorModifierCoreMetadata& InModifierMetadata)->bool
			{
				return InModifierMetadata.IsAllowedBefore(InMoveOp.MoveModifier->GetModifierName());
			});

			// do we allow the move modifier to be after the current modifier
			bAllowedBefore &= ModifierSubsystem->ProcessModifierMetadata(InMoveOp.MoveModifier->GetModifierName(), [InModifier](const FActorModifierCoreMetadata& InModifierMetadata)->bool
			{
				return InModifierMetadata.IsAllowedAfter(InModifier->GetModifierName());
			});

			if (!bAllowedBefore)
			{
				if (InMoveOp.FailReason)
				{
					*InMoveOp.FailReason = FText::Format(
						LOCTEXT("UnallowedMoveModifierBefore", "Modifier {0} is not allowed before modifier {1}"),
						FText::FromName(InModifier->GetModifierName()),
						FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
				}

				return false;
			}

			OutRequiredModifiers.Remove(InModifier);
		}
		else
		{
			// do we allow current modifier to be after the move modifier
			bool bAllowedAfter = ModifierSubsystem->ProcessModifierMetadata(InModifier->GetModifierName(), [&InMoveOp](const FActorModifierCoreMetadata& InModifierMetadata)->bool
			{
				return InModifierMetadata.IsAllowedAfter(InMoveOp.MoveModifier->GetModifierName());
			});

			// do we allow the move modifier to be before the current modifier
			bAllowedAfter &= ModifierSubsystem->ProcessModifierMetadata(InMoveOp.MoveModifier->GetModifierName(), [InModifier](const FActorModifierCoreMetadata& InModifierMetadata)->bool
			{
				return InModifierMetadata.IsAllowedBefore(InModifier->GetModifierName());
			});

			if (!bAllowedAfter)
			{
				if (InMoveOp.FailReason)
				{
					*InMoveOp.FailReason = FText::Format(
						LOCTEXT("UnallowedMoveModifierAfter", "Modifier {0} is not allowed after modifier {1}"),
						FText::FromName(InModifier->GetModifierName()),
						FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
				}

				return false;
			}

			OutDependentModifiers.Remove(InModifier);
		}

		return true;
	});

	if (!bMoveAllowed)
	{
		return false;
	}

	if (!OutRequiredModifiers.IsEmpty())
	{
		FString RequiredModifierWarning;

		for (const UActorModifierCoreBase* RequiredModifier : OutRequiredModifiers)
		{
			if (!RequiredModifierWarning.IsEmpty())
			{
				RequiredModifierWarning += TEXT(", ");
			}
			RequiredModifierWarning += RequiredModifier->GetModifierName().ToString();
		}

		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = FText::Format(
				LOCTEXT("MoveModifierRequiredLeft", "Required modifier(s) {0} should precede modifier {1}"),
				FText::FromString(RequiredModifierWarning),
				FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
		}

		return false;
	}

	if (!OutDependentModifiers.IsEmpty())
	{
		FString DependentModifierWarning;

		for (const UActorModifierCoreBase* DependentModifier : OutDependentModifiers)
		{
			if (!DependentModifierWarning.IsEmpty())
			{
				DependentModifierWarning += TEXT(", ");
			}
			DependentModifierWarning += DependentModifier->GetModifierName().ToString();
		}

		if (InMoveOp.FailReason)
		{
			*InMoveOp.FailReason = FText::Format(
				LOCTEXT("MoveModifierDependentLeft", "Dependent modifier(s) {0} should follow modifier {1}"),
				FText::FromString(DependentModifierWarning),
				FText::FromName(InMoveOp.MoveModifier->GetModifierName()));
		}

		return false;
	}

#if WITH_EDITOR
	Modify();
	InMoveOp.MoveModifier->Modify();
#endif

	Modifiers.Remove(InMoveOp.MoveModifier);

	// If we delete and we move after current idx, we need to subtract one
	const bool bMoveModifierBeforeCurrentIdx = MoveIdx < CurrentIdx;
	MoveIdx = bMoveModifierBeforeCurrentIdx ? MoveIdx : MoveIdx - 1;

	Modifiers.Insert(InMoveOp.MoveModifier, MoveIdx);

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Move modifier %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *InMoveOp.MoveModifier->GetModifierName().ToString());

	OnModifierMovedDelegate.Broadcast(InMoveOp.MoveModifier);

	// update the stack to restore to initial state
	MarkModifierDirty();

	// Check stack optimization
	ScheduleModifierOptimization(true);

	return true;
}

bool UActorModifierCoreStack::RemoveModifier(FActorModifierCoreStackRemoveOp& InRemoveOp)
{
	if (IsModifierStackFrozen())
	{
		if (InRemoveOp.FailReason)
		{
			*InRemoveOp.FailReason = LOCTEXT("RemoveFailed_FrozenStack", "Modifier Stack is Frozen");
		}
		return false;
	}

	if (!IsValid(InRemoveOp.RemoveModifier))
	{
		if (InRemoveOp.FailReason)
		{
			*InRemoveOp.FailReason = LOCTEXT("InvalidModifierProvided", "Modifier provided is not valid");
		}
		return false;
	}

	// is the modifier in this stack
	if (InRemoveOp.RemoveModifier->GetModifierStack() != this || !Modifiers.Contains(InRemoveOp.RemoveModifier))
	{
		if (InRemoveOp.FailReason)
		{
			*InRemoveOp.FailReason = LOCTEXT("DifferentModifierStack", "Modifier provided is not inside this stack");
		}
		return false;
	}

	TSet<UActorModifierCoreBase*> RemoveModifiers;
	if (InRemoveOp.bRemoveDependencies)
	{
		GetRequiredModifiers(InRemoveOp.RemoveModifier,RemoveModifiers);
	}
	RemoveModifiers.Add(InRemoveOp.RemoveModifier);

	// loop reverse order and add modifiers that needs to be removed in correct order
	TArray<TWeakObjectPtr<UActorModifierCoreBase>> RemoveModifiersWeak;
	for (int32 ModifierIdx = Modifiers.Num() - 1; ModifierIdx >= 0; ModifierIdx--)
	{
		TObjectPtr<UActorModifierCoreBase>& Modifier = Modifiers[ModifierIdx];

		// do we remove this modifier
		if (RemoveModifiers.Contains(Modifier))
		{
			// get required modifiers
			TSet<UActorModifierCoreBase*> OutDependentModifiers;
			if (GetDependentModifiers(Modifier, OutDependentModifiers))
			{
				// do we also remove this modifier dependency
				const TSet<UActorModifierCoreBase*> DependentModifiersLeft = OutDependentModifiers.Difference(RemoveModifiers);
				if (DependentModifiersLeft.IsEmpty())
				{
					RemoveModifiersWeak.Add(Modifier);
				}
				else
				{
					FString DependentModifiersWarning;
					for (const UActorModifierCoreBase* ModifierLeft : DependentModifiersLeft)
					{
						if (!DependentModifiersWarning.IsEmpty())
						{
							DependentModifiersWarning += TEXT(", ");
						}
						DependentModifiersWarning += ModifierLeft->GetModifierName().ToString();
					}

					if (InRemoveOp.FailReason)
					{
						*InRemoveOp.FailReason = FText::Format(
							LOCTEXT("ModifierRequiredByModifier", "Modifier(s) ({0}) depend on modifier {1}"),
							FText::FromString(DependentModifiersWarning),
							FText::FromName(Modifier->GetModifierName()));
					}
					return false;
				}
			}
		}
	}

	// do we have modifiers to remove
	if (RemoveModifiersWeak.IsEmpty())
	{
		if (InRemoveOp.FailReason)
		{
			*InRemoveOp.FailReason = LOCTEXT("RemoveModifiers", "Cannot remove all the modifiers provided from this stack");
		}
		return false;
	}

#if WITH_EDITOR
	Modify();
#endif

	for (TWeakObjectPtr<UActorModifierCoreBase>& RemoveModifier : RemoveModifiersWeak)
	{

#if WITH_EDITOR
		RemoveModifier->Modify();
#endif

		Modifiers.Remove(RemoveModifier.Get());

		if (InRemoveOp.RemovedDependencies && RemoveModifier != InRemoveOp.RemoveModifier)
		{
			InRemoveOp.RemovedDependencies->Add(RemoveModifier.Get());
		}
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Remove modifier %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *InRemoveOp.RemoveModifier->GetModifierName().ToString());

	// update the stack to restore to initial state
	MarkModifierDirty();

	// Check stack optimization
	ScheduleModifierOptimization(true);

	// wait for idle to remove remaining refs and un-initialize modifiers
	TWeakObjectPtr<UActorModifierCoreStack> ThisWeak(this);
	ProcessFunctionOnIdle([ThisWeak, RemoveModifiersWeak]()
	{
		UActorModifierCoreStack* This = ThisWeak.Get();
		if (!IsValid(This))
		{
			return;
		}

		for (const TWeakObjectPtr<UActorModifierCoreBase>& RemoveModifierWeak : RemoveModifiersWeak)
		{
			UActorModifierCoreBase* RemoveModifier = RemoveModifierWeak.Get();

			if (This && RemoveModifier)
			{
				if (This->IsModifierStackFrozen())
				{
					// Broadcast the removal directly without going through uninit process
					UActorModifierCoreStack::OnModifierRemovedDelegate.Broadcast(RemoveModifier, EActorModifierCoreDisableReason::User);
				}
				else
				{
					RemoveModifier->UninitializeModifier(EActorModifierCoreDisableReason::User);
				}

				This->ExecutionTask.RemoveModifier(*RemoveModifier);

				RemoveModifier->MarkAsGarbage();
			}
		}
	});

	// number of modifiers to be removed from this stack
	return RemoveModifiersWeak.Num() > 0;
}

bool UActorModifierCoreStack::GetDependentModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutDependentModifiers) const
{
	if (!IsValid(InModifier))
	{
		return false;
	}

	// is the modifier in this stack
	const int32 ModifierIdx = Modifiers.Find(InModifier);
	if (ModifierIdx == INDEX_NONE)
	{
		return false;
	}

	// get modifier subsystem to detect dependency
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	if (!ModifierSubsystem)
	{
		return false;
	}

	// loop in reverse to find dependent modifiers
	for (int32 Idx = Modifiers.Num() - 1; Idx > ModifierIdx; Idx--)
	{
		if (const TObjectPtr<UActorModifierCoreBase>& CurModifier = Modifiers[Idx])
		{
			const FName CurModifierName = CurModifier->GetModifierName();

			const bool bRequired = ModifierSubsystem->ProcessModifierMetadata(CurModifierName, [InModifier](const FActorModifierCoreMetadata& InMetadata)->bool
			{
				const TConstArrayView<FName> ModifierRequired = InMetadata.GetDependencies();
				return ModifierRequired.Contains(InModifier->GetModifierName());
			});

			if (bRequired)
			{
				OutDependentModifiers.Add(CurModifier);
			}
		}
	}

	return true;
}

bool UActorModifierCoreStack::GetRequiredModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutRequiredModifiers) const
{
	if (!IsValid(InModifier))
	{
		return false;
	}

	// is the modifier in this stack
	const int32 ModifierIdx = Modifiers.Find(InModifier);
	if (ModifierIdx == INDEX_NONE)
	{
		return false;
	}

	// get modifier subsystem to detect dependency
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	if (!ModifierSubsystem)
	{
		return false;
	}

	TArray<FName> OutModifiersNames;
	if (!ModifierSubsystem->BuildModifierDependencies(InModifier->GetModifierName(), OutModifiersNames))
	{
		return false;
	}

	// loop to find required modifiers
	for (int32 Idx = 0; Idx < ModifierIdx; Idx++)
	{
		if (const TObjectPtr<UActorModifierCoreBase>& CurModifier = Modifiers[Idx])
		{
			if (OutModifiersNames.Contains(CurModifier->GetModifierName()))
			{
				OutRequiredModifiers.Add(CurModifier);
			}
		}
	}

	return true;
}

bool UActorModifierCoreStack::RemoveAllModifiers()
{
	LockModifierExecution();

	// Temporarily unfreeze modifier stack to allow removing all modifiers
	// ProcessIdleFunction checks for IsModifierStackFrozen when this goes out of scope. And it's ok.
	// This is only here to overcome the first check blocking removal when stack is frozen
	TGuardValue<bool> StackUnfrozenScope(bModifierStackFrozen, false);

	for (int32 ModifierIdx = Modifiers.Num() - 1; ModifierIdx >= 0; ModifierIdx--)
	{
		FActorModifierCoreStackRemoveOp RemoveOp;
		RemoveOp.RemoveModifier = Modifiers[ModifierIdx];

		RemoveModifier(RemoveOp);
	}

	UnlockModifierExecution();

	return Modifiers.IsEmpty();
}

void UActorModifierCoreStack::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	if (IsModifierStackFrozen())
	{
		return;
	}

	// Restore in reverse order
	for (int32 ModifierIdx = Modifiers.Num() - 1; ModifierIdx >= 0; ModifierIdx--)
	{
		if (const TObjectPtr<UActorModifierCoreBase>& Modifier = Modifiers[ModifierIdx])
		{
			Modifier->Unapply();

			if (Modifier->bModifierEnabled)
			{
				Modifier->DisableModifier(InReason);
			}
		}
	}
}

void UActorModifierCoreStack::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	if (!IsModifierInitialized())
	{
		return;
	}

	// Enable in correct order
	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		if (Modifier && Modifier->bModifierEnabled)
		{
			Modifier->EnableModifier(InReason);
		}
	}
}

void UActorModifierCoreStack::OnModifierStackFrozenChanged()
{
	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Modifier stack %ls"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, IsModifierStackFrozen() ? TEXT("frozen") : TEXT("unfrozen"));

	// Re-execute stack if modifier is no longer frozen
	if (!IsModifierStackFrozen())
	{
		MarkModifierDirty();
	}
}

void UActorModifierCoreStack::OnActorTransformUpdated(USceneComponent*, EUpdateTransformFlags, ETeleportType)
{
	if (IsModifierEnabled())
	{
		OnModifiedActorTransformed();
	}
}

void UActorModifierCoreStack::OnModifierExecutionFinished(bool bInResult)
{
	bAllModifiersDirty = false;

	if (bInResult)
	{
		Next();
	}
	else
	{
		Fail(LOCTEXT("ModifierStackExecutionFail", "A modifier in this stack failed to execute"));
	}

	if (IsRootStack() && RootExecutionTask.IsSet())
	{
		if (bInResult)
		{
			RootExecutionTask->Next();
		}
		else
		{
			RootExecutionTask->Fail();
		}
		
		RootExecutionTask.Reset();

		// process idle function
		if (!IsModifierExecutionLocked())
		{
			ScheduleModifierOptimization(false);

			for (int32 Index = 0; Index < OnIdleFunctions.Num(); ++Index)
			{
				OnIdleFunctions[Index]();
			}

			OnIdleFunctions.Empty();
		}
	}
}

void UActorModifierCoreStack::ScheduleModifierOptimization(bool bInInvalidateAll)
{
	if (!IsRootStack())
	{
		return;
	}

	TWeakObjectPtr<UActorModifierCoreStack> ThisWeak(this);
	ProcessFunctionOnIdle([ThisWeak, bInInvalidateAll]()
	{
		UActorModifierCoreStack* Stack = ThisWeak.Get();

		if (!IsValid(Stack))
		{
			return;
		}

		Stack->CheckModifierOptimization(bInInvalidateAll);
	});
}

void UActorModifierCoreStack::CheckModifierOptimization(bool bInInvalidateAll)
{
	if (!IsRootStack() || !IsModifierIdle())
	{
		return;
	}

	if (bInInvalidateAll)
	{
		bModifierOptimized = false;
	}

	if (bModifierOptimized)
	{
		return;
	}

	TArray<UActorModifierCoreBase*> AllModifiers;
	ProcessFunction([&AllModifiers, bInInvalidateAll](const UActorModifierCoreBase* InModifier)
	{
		if (!InModifier->IsModifierStack())
		{
			UActorModifierCoreBase* Modifier = const_cast<UActorModifierCoreBase*>(InModifier);

			if (bInInvalidateAll)
			{
				Modifier->bModifierOptimized = false;
			}

			AllModifiers.Add(Modifier);
		}

		return true;
	});

	TSet<FName> Categories;
	Categories.Reserve(AllModifiers.Num());

	TSet<UActorModifierCoreBase*> UnoptimizedModifiers;
	UnoptimizedModifiers.Reserve(AllModifiers.Num());

	for (UActorModifierCoreBase* Modifier : AllModifiers)
	{
		if (!Modifier->bModifierOptimized)
		{
			const FActorModifierCoreMetadata& ModifierMetadata = Modifier->GetModifierMetadata();

			for (const FName& Category : Categories)
			{
				if (ModifierMetadata.ShouldAvoidAfter(Category))
				{
					UnoptimizedModifiers.Add(Modifier);

					const FText Message = FText::Format(LOCTEXT("AvoidAfterCategory", "Should be moved above {0} modifiers"), FText::FromName(Category));
					Modifier->Status = FActorModifierCoreStatus(EActorModifierCoreStatus::Warning, Message);

					// Only log when we invalidate all and recheck optimization
					if (bInInvalidateAll)
					{
						UE_LOGF(LogActorModifierCore, Log, "[%ls][%ls][%ls] Optimization possible for modifier %ls : %ls"
							, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
							, *GetNameSafe(GetClass())
							, *GetModifierName().ToString()
							, *Modifier->GetModifierName().ToString()
							, *Message.ToString());
					}

					break;
				}
			}
		}

		Categories.Add(Modifier->GetModifierCategory());
	}

	Categories.Empty(AllModifiers.Num());
	for (UActorModifierCoreBase* Modifier : ReverseIterate(AllModifiers))
	{
		if (!Modifier->bModifierOptimized)
		{
			const FActorModifierCoreMetadata& ModifierMetadata = Modifier->GetModifierMetadata();

			for (const FName& Category : Categories)
			{
				if (ModifierMetadata.ShouldAvoidBefore(Category))
				{
					UnoptimizedModifiers.Add(Modifier);

					const FText Message = FText::Format(LOCTEXT("AvoidBeforeCategory", "Should be moved below {0} modifiers"), FText::FromName(Category));
					Modifier->Status = FActorModifierCoreStatus(EActorModifierCoreStatus::Warning, Message);

					// Only log when we invalidate all and recheck optimization
					if (bInInvalidateAll)
					{
						UE_LOGF(LogActorModifierCore, Log, "[%ls][%ls][%ls] Optimization possible for modifier %ls : %ls"
							, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
							, *GetNameSafe(GetClass())
							, *GetModifierName().ToString()
							, *Modifier->GetModifierName().ToString()
							, *Message.ToString());
					}

					break;
				}
			}

			Modifier->bModifierOptimized = !UnoptimizedModifiers.Contains(Modifier);
		}

		Categories.Add(Modifier->GetModifierCategory());
	}

	bModifierOptimized = UnoptimizedModifiers.IsEmpty();
}

void UActorModifierCoreStack::OnBlueprintModifierReplaced(UActorModifierCoreBlueprintBase* InOldModifier, UActorModifierCoreBlueprintBase* InNewModifier)
{
	const int32 ModifierIndex = Modifiers.Find(InOldModifier);

	if (ModifierIndex == INDEX_NONE)
	{
		return;
	}

	// Init metadata for new modifier
	InNewModifier->PostModifierCreation(this);

	// Copy non uproperty var
	InNewModifier->bModifierApplied = InOldModifier->bModifierApplied;
	InNewModifier->bModifierInitialized = InOldModifier->bModifierInitialized;
	InNewModifier->bModifierOptimized = InOldModifier->bModifierOptimized;

	AActor* TargetActor = GetModifiedActor();

	// Check if support rule has changed
	const bool bActorSupported = InNewModifier->GetModifierMetadata().IsCompatibleWith(TargetActor);

	if (bActorSupported)
	{
		Modifiers[ModifierIndex] = InNewModifier;
	}
	else
	{
		Modifiers.RemoveAt(ModifierIndex);
	}

	// Replace old modifier by new modifier in stack
	ExecutionTask.ReplaceModifier(*InOldModifier, *InNewModifier);

	InNewModifier->OnModifierReplacedEvent(TargetActor);

	OnModifierReplacedDelegate.Broadcast(InNewModifier);

	InNewModifier->MarkModifierDirty();
}

void UActorModifierCoreStack::RestorePreState()
{
	if (IsModifierStackFrozen())
	{
		return;
	}

	// Restore previously applied modifiers
	ExecutionTask.Restore();

	// Stack is restored, execute one time callbacks
	for (int32 Index = 0; Index < OnRestoreFunctions.Num(); ++Index)
	{
		OnRestoreFunctions[Index]();
	}
	OnRestoreFunctions.Empty();
}

void UActorModifierCoreStack::Apply()
{
	if (IsModifierStackFrozen())
	{
		return;
	}

	// restore original state before dirty modifiers
	RestorePreState();

	if (bModifierInitialized)
	{
		ExecutionTask.Apply(GetModifiers(), this);
	}
}

void UActorModifierCoreStack::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	if (UActorModifierCoreSubsystem* Subsystem = UActorModifierCoreSubsystem::Get())
	{
		if (IsRootStack())
		{
			if (AActor* TargetActor = GetModifiedActor())
			{
				// unregister stack from actor in subsystem
				Subsystem->UnregisterActorModifierStack(TargetActor);

				if (USceneComponent* RootComponent = TargetActor->GetRootComponent())
				{
					RootComponent->TransformUpdated.RemoveAll(this);
				}
			}
		}

		Subsystem->OnModifierReplaced().RemoveAll(this);
	}

	if (IsModifierStackFrozen())
	{
		for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
		{
			if (Modifier)
			{
				// Broadcast the removal directly without going through uninit process
				UActorModifierCoreStack::OnModifierRemovedDelegate.Broadcast(Modifier, InReason);
			}
		}
	}
	else
	{
		// un-initialize inner modifiers if they are not un-initialized
		for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
		{
			if (Modifier)
			{
				Modifier->UninitializeModifier(InReason);
			}
		}
	}

}

void UActorModifierCoreStack::OnModifiedActorTransformed()
{
	if (IsModifierStackFrozen())
	{
		return;
	}

	for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
	{
		// only tell enable modifier that our actor has moved
		if (Modifier && Modifier->IsModifierEnabled() && Modifier->IsModifierIdle())
		{
			Modifier->OnModifiedActorTransformed();
		}
	}
}

void UActorModifierCoreStack::OnModifierDirty(UActorModifierCoreBase* DirtyModifier, bool bExecute)
{
	// Run checks on dirty modifier
	if (!DirtyModifier
		|| !DirtyModifier->IsModifierInitialized()
		|| DirtyModifier->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed)
		|| DirtyModifier->GetRootModifierStack() != GetRootModifierStack())
	{
		return;
	}

	Super::OnModifierDirty(DirtyModifier, bExecute);

	// mark stack as dirty
	bModifierDirty = true;

	// If we are the one dirty then re-execute everything
	if (DirtyModifier == this)
	{
		bAllModifiersDirty = true;
	}

	// Only the root stack handles updates
	if (!IsRootStack()
		|| HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed)
		|| !IsValid(GetModifiedActor()))
	{
		return;
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Modifier %ls marked dirty"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString()
		, *DirtyModifier->GetModifierName().ToString());

	if (!bExecute
		|| IsModifierStackFrozen()
		|| DirtyModifier->IsModifierExecutionLocked()
		|| !IsModifierStackInitialized()
		|| !IsModifierDirty()
		|| !IsModifierReady()
		|| !IsModifierIdle()
		|| IsModifierExecutionLocked())
	{
		return;
	}

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Executing modifier root stack"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString());

	// Execute root stack as standalone modifier
	RootExecutionTask = FActorModifierCoreExecutionTask();
	RootExecutionTask->Apply({this}, nullptr);
}

void UActorModifierCoreStack::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.AllowTick(true);
	InMetadata.SetName(TEXT("Stack"));
}

void UActorModifierCoreStack::TickModifier(float InDelta) const
{
	if (IsModifierStackFrozen() 
		|| !IsRootStack()
		|| !IsModifierEnabled()
		|| !IsModifierInitialized()
		|| !IsModifierIdle()
		|| !Metadata->IsTickAllowed()
		|| IsModifierDirty())
	{
		return;
	}

	// Skips nested stack when processing
	UActorModifierCoreBase* FirstDirtyModifier = nullptr;
	ProcessFunction([&FirstDirtyModifier](const UActorModifierCoreBase* InModifier)->bool
	{
		if (IsValid(InModifier)
			&& InModifier->IsModifierEnabled()
			&& !InModifier->IsModifierDirty()
			&& InModifier->Metadata->IsTickAllowed()
			&& InModifier->IsModifierDirtyable())
		{
			FirstDirtyModifier = const_cast<UActorModifierCoreBase*>(InModifier);
			// break loop
			return false;
		}

		return true;
	});

	if (FirstDirtyModifier)
	{
		FirstDirtyModifier->MarkModifierDirty();
	}
}

#undef LOCTEXT_NAMESPACE
