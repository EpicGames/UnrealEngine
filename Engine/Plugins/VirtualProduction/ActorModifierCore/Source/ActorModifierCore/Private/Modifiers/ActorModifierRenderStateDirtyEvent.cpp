// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "HAL/IConsoleManager.h"

namespace UE::ActorModifierCore
{
namespace Private
{

/**
 * Determines whether render state reason should be enabled.
 * If disabled, IsRenderStateDirtyRelevant() will always return true (no specific reason to otherwise make it irrelevant)
 */
static bool bRenderStateDirtyReasonEnabled = false;
static FAutoConsoleVariableRef CVarRenderStateDirtyReasonEnabled(
	TEXT("ActorModifierCore.EnableRenderStateDirtyReason"), 
	bRenderStateDirtyReasonEnabled,
	TEXT("If true some render state dirty events will have a reason to them and modifiers will skip work if the reason is deemed irrelevant by the modifier."),
	ECVF_Default);

// Defaults to unknown as most MarkRenderStateDirty calls will not have a reason scope to it
static ERenderStateDirtyReason RenderStateDirtyReason = ERenderStateDirtyReason::Unknown;

} // UE::ActorModifierCore::Private

ERenderStateDirtyReason GetRenderStateDirtyReason()
{
	return Private::RenderStateDirtyReason;
}

bool IsRenderStateDirtyRelevant(ERenderStateDirtyReason InReason)
{
	return !Private::bRenderStateDirtyReasonEnabled
		|| Private::RenderStateDirtyReason == ERenderStateDirtyReason::Unknown
		|| EnumHasAnyFlags(Private::RenderStateDirtyReason, InReason); 
}

FRenderStateDirtyReasonScope::FRenderStateDirtyReasonScope(ERenderStateDirtyReason InReason)
	: PreviousReason(Private::RenderStateDirtyReason)
{
	Private::RenderStateDirtyReason = InReason;
}

FRenderStateDirtyReasonScope::~FRenderStateDirtyReasonScope()
{
	Private::RenderStateDirtyReason = PreviousReason;
}

} //  UE::ActorModifierCore
