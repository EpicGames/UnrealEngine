// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "ActorModifierCoreLog.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreBlueprintBase"

void UActorModifierCoreBlueprintBase::FlagModifierDirty()
{
	if (!IsTemplate())
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier flagged dirty"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());

		MarkModifierDirty(/** Execute */true);
	}
}

#if WITH_EDITOR
void UActorModifierCoreBlueprintBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	// Only trigger modifier if not interactive and property owned by children of this class
	if (InPropertyChangedEvent.Property
		&& InPropertyChangedEvent.Property->GetOwnerClass()
		&& InPropertyChangedEvent.Property->GetOwnerClass()->IsChildOf<UActorModifierCoreBlueprintBase>()
		&& InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		FlagModifierDirty();
	}
}
#endif

void UActorModifierCoreBlueprintBase::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier setup"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString());

	// Since the blueprint utility functions returns a copy of the struct, make sure we get a result
	OnModifierSetupEvent(InMetadata, /** OutMetadata */InMetadata);

	if (InMetadata.GetName().IsNone())
	{
		UE_LOGF(LogActorModifierCore, Warning, "[%ls][%ls][%ls] Blueprint modifier setup failed : Name was not defined"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());
	}
}

void UActorModifierCoreBlueprintBase::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier added with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		OnModifierAddedEvent(TargetActor, InReason);
	}
}

void UActorModifierCoreBlueprintBase::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier enabled with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		OnModifierEnabledEvent(TargetActor, InReason);
	}
}

void UActorModifierCoreBlueprintBase::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier disabled with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		OnModifierDisabledEvent(TargetActor, InReason);
	}
}

void UActorModifierCoreBlueprintBase::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier removed with reason %i"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString()
			, EnumToUnderlyingType(InReason));

		OnModifierRemovedEvent(TargetActor, InReason);
	}
}

void UActorModifierCoreBlueprintBase::SavePreState()
{
	Super::SavePreState();

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier save pre state"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());

		OnModifierSaveStateEvent(TargetActor);
	}
}

void UActorModifierCoreBlueprintBase::RestorePreState()
{
	Super::RestorePreState();

	AActor* TargetActor = GetModifiedActor();

	if (IsValid(TargetActor))
	{
		UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier restore pre state"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());

		OnModifierRestoreStateEvent(TargetActor);
	}
}

void UActorModifierCoreBlueprintBase::Apply()
{
	FText FailReason = FText::GetEmpty();
	AActor* TargetActor = GetModifiedActor();

	UE_LOGF(LogActorModifierCore, Verbose, "[%ls][%ls][%ls] Blueprint modifier apply"
		, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
		, *GetNameSafe(GetClass())
		, *GetModifierName().ToString());

	if (IsValid(TargetActor) && OnModifierApplyEvent(TargetActor, FailReason))
	{
		Next();
	}
	else
	{
		// Fail reason must be set
		if (FailReason.IsEmpty())
		{
			FailReason = FText::Format(LOCTEXT("ApplyFailed", "{0} : Blueprint modifier {1} apply failed"), FText::FromString(TargetActor ? TargetActor->GetActorNameOrLabel() : TEXT("?")), FText::FromName(GetModifierName()));
		}

		Fail(FailReason);
	}
}

#undef LOCTEXT_NAMESPACE
