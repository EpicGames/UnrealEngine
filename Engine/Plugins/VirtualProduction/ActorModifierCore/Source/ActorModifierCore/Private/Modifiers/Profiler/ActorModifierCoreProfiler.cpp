// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Profiler/ActorModifierCoreProfiler.h"

#include "ObjectTrace.h"
#include "Modifiers/ActorModifierCoreBase.h"

UE_TRACE_CHANNEL_DEFINE(ModifierChannel)

UE_TRACE_EVENT_BEGIN(ModifierProfiler, EndExecutionEvent)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Tag)
	UE_TRACE_EVENT_FIELD(double, ExecutionTime)
	UE_TRACE_EVENT_FIELD(double, AverageExecutionTime)
	UE_TRACE_EVENT_FIELD(double, TotalExecutionTime)
	UE_TRACE_EVENT_FIELD(uint64, ExecutionCount)
	UE_TRACE_EVENT_FIELD(uint32, FrameCountDelta)
	UE_TRACE_EVENT_FIELD(uint32, FrameRateDelta)
UE_TRACE_EVENT_END()

FActorModifierCoreProfiler::~FActorModifierCoreProfiler()
{
}

void FActorModifierCoreProfiler::SetupProfilingStats()
{
	ProfilerStats.AddProperty(ExecutionTimeName, EPropertyBagPropertyType::Double);
	ProfilerStats.AddProperty(AverageExecutionTimeName, EPropertyBagPropertyType::Double);
	ProfilerStats.AddProperty(TotalExecutionTimeName, EPropertyBagPropertyType::Double);
	ProfilerStats.AddProperty(FrameCountDeltaName, EPropertyBagPropertyType::Int64);
	ProfilerStats.AddProperty(FrameRateDeltaName, EPropertyBagPropertyType::Float);
}

void FActorModifierCoreProfiler::BeginProfiling()
{
	{
		TRACE_BOOKMARK(TEXT("BeginProfiling_%s"), *GetProfilerTag())
		TRACE_OBJECT(GetModifier())
		TRACE_BEGIN_REGION(*GetProfilerTag())
	}

	ExecutionTimeStart = FPlatformTime::Cycles64();

	extern ENGINE_API float GAverageFPS;
	FrameCountStart = GFrameCounter;
	FrameRateStart = GAverageFPS;
}

void FActorModifierCoreProfiler::EndProfiling()
{
	const double ExecutionTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - ExecutionTimeStart);

	TotalExecutionTime += ExecutionTime;
	ExecutionCount++;

	ProfilerStats.SetValueDouble(ExecutionTimeName, ExecutionTime);
	ProfilerStats.SetValueDouble(TotalExecutionTimeName, TotalExecutionTime);

	const double AverageExecutionTime = TotalExecutionTime / (ExecutionCount * 1.f);
	ProfilerStats.SetValueDouble(AverageExecutionTimeName, AverageExecutionTime);

	extern ENGINE_API float GAverageFPS;
	const int64 FrameCountDelta = GFrameCounter - FrameCountStart;
	const int64 FrameRateDelta = GAverageFPS - FrameRateStart;
	ProfilerStats.SetValueInt64(FrameCountDeltaName, FrameCountDelta);
	ProfilerStats.SetValueFloat(FrameRateDeltaName, FrameRateDelta);

	{
		TRACE_END_REGION(*GetProfilerTag())
		TRACE_OBJECT(GetModifier())
		UE_TRACE_LOG(ModifierProfiler, EndExecutionEvent, ModifierChannel)
			<< EndExecutionEvent.Tag(*GetProfilerTag())
			<< EndExecutionEvent.ExecutionTime(ExecutionTime)
			<< EndExecutionEvent.AverageExecutionTime(AverageExecutionTime)
			<< EndExecutionEvent.TotalExecutionTime(TotalExecutionTime)
			<< EndExecutionEvent.ExecutionCount(ExecutionCount)
			<< EndExecutionEvent.FrameCountDelta(FrameCountDelta)
			<< EndExecutionEvent.FrameRateDelta(FrameRateDelta);
		TRACE_BOOKMARK(TEXT("EndProfiling_%s"), *GetProfilerTag())
	}
}

TSet<FName> FActorModifierCoreProfiler::GetMainProfilingStats() const
{
	return TSet<FName>
	{
		ExecutionTimeName,
		AverageExecutionTimeName,
		TotalExecutionTimeName
	};
}

AActor* FActorModifierCoreProfiler::GetModifierActor() const
{
	return GetModifier() ? GetModifier()->GetModifiedActor() : nullptr;
}

double FActorModifierCoreProfiler::GetExecutionTime() const
{
	TValueOrError<double, EPropertyBagResult> ValueResult = ProfilerStats.GetValueDouble(ExecutionTimeName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0.f;
}

double FActorModifierCoreProfiler::GetAverageExecutionTime() const
{
	TValueOrError<double, EPropertyBagResult> ValueResult = ProfilerStats.GetValueDouble(AverageExecutionTimeName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0.f;
}

double FActorModifierCoreProfiler::GetTotalExecutionTime() const
{
	TValueOrError<double, EPropertyBagResult> ValueResult = ProfilerStats.GetValueDouble(TotalExecutionTimeName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0.f;
}

int64 FActorModifierCoreProfiler::GetFrameCountDelta() const
{
	TValueOrError<int64, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt64(FrameCountDeltaName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

float FActorModifierCoreProfiler::GetFrameRateDelta() const
{
	TValueOrError<float, EPropertyBagResult> ValueResult = ProfilerStats.GetValueFloat(FrameRateDeltaName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0.f;
}

void FActorModifierCoreProfiler::ConstructInternal(UActorModifierCoreBase* InModifier, const FName& InProfilerType)
{
	ModifierWeak = InModifier;
	ProfilerType = InProfilerType;

	// Generate tag for this profiler
	{
		// Sanitize profiler tag due to GetGeneratedTypeName usage
		FString ProfilerTypeStr = InProfilerType.ToString();
		if (ProfilerTypeStr.StartsWith("class "))
		{
			ProfilerTypeStr.RemoveFromStart("class ");
		}

		const UActorModifierCoreBase* Modifier = GetModifier();
		const AActor* Actor = GetModifierActor();

		const FString ActorName = Actor ? Actor->GetActorNameOrLabel() : TEXT("");
		const FString ModifierName = Modifier ? Modifier->GetModifierName().ToString() : TEXT("");
		const FString Hash = FString::Printf(TEXT("%u"), HashCombine(GetTypeHash(Actor), GetTypeHash(Modifier)));

		ProfilerTag = ProfilerTypeStr + TEXT("_") + ActorName + TEXT("_") + ModifierName + TEXT("_") + Hash;
	}

	SetupProfilingStats();

	// Make properties read only and export to struct
	{
		const FStructView StructView = ProfilerStats.GetMutableValue();
		for (TFieldIterator<FProperty> PropertyIterator(StructView.GetScriptStruct(), EFieldIteratorFlags::ExcludeSuper); PropertyIterator; ++PropertyIterator)
		{
			FProperty* Property = *PropertyIterator;
			Property->SetPropertyFlags(CPF_EditConst);
		}

		StructProfilerStats = MakeShared<FStructOnScope>(StructView.GetScriptStruct(), StructView.GetMemory());
	}
}
