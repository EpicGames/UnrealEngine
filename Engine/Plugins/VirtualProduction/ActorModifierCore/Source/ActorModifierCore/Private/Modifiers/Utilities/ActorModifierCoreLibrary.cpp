// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Utilities/ActorModifierCoreLibrary.h"

#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreLibrary, Log, All);

void UActorModifierCoreLibrary::MarkModifierDirty(UActorModifierCoreBase* InModifier)
{
	if (InModifier)
	{
		InModifier->MarkModifierDirty();
	}
}

bool UActorModifierCoreLibrary::FindModifierStack(AActor* InActor, UActorModifierCoreStack*& OutModifierStack, bool bInCreateIfNone)
{
	OutModifierStack = nullptr;
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InActor) || !ModifierSubsystem)
	{
		return false;
	}

	OutModifierStack = ModifierSubsystem->GetActorModifierStack(InActor);

	if (!OutModifierStack && bInCreateIfNone)
	{
		OutModifierStack = ModifierSubsystem->AddActorModifierStack(InActor);
	}

	return !!OutModifierStack;
}

bool UActorModifierCoreLibrary::InsertModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreInsertOperation& InOperation, UActorModifierCoreBase*& OutNewModifier)
{
	OutNewModifier = nullptr;
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InModifierStack) || !IsValid(InModifierStack->GetModifiedActor()) || !ModifierSubsystem)
	{
		return false;
	}

	FText FailReason = FText::GetEmpty();
	FActorModifierCoreStackInsertOp InsertOp;
	InsertOp.InsertPosition = InOperation.InsertPosition;
	InsertOp.InsertPositionContext = InOperation.InsertPositionContext;
	InsertOp.FailReason = &FailReason;

	if (!GetModifierNameByClass(InOperation.ModifierClass, InsertOp.NewModifierName))
	{
		return false;
	}

	OutNewModifier = ModifierSubsystem->InsertModifier(InModifierStack, InsertOp);

	if (!FailReason.IsEmpty())
	{
		UE_LOGF(LogActorModifierCoreLibrary, Warning, "InsertModifier %ls failing reason : %ls", *InsertOp.NewModifierName.ToString(), *FailReason.ToString())
	}

	return !!OutNewModifier;
}

bool UActorModifierCoreLibrary::CloneModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreCloneOperation& InOperation, UActorModifierCoreBase*& OutNewModifier)
{
	OutNewModifier = nullptr;
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InModifierStack) || !IsValid(InModifierStack->GetModifiedActor()) || !ModifierSubsystem)
	{
		return false;
	}

	FText FailReason = FText::GetEmpty();
	FActorModifierCoreStackCloneOp CloneOp;
	CloneOp.ClonePosition = InOperation.ClonePosition;
	CloneOp.ClonePositionContext = InOperation.ClonePositionContext;
	CloneOp.FailReason = &FailReason;

	if (!IsValid(InOperation.CloneModifier))
	{
		return false;
	}

	TArray<UActorModifierCoreBase*> NewModifiers = ModifierSubsystem->CloneModifiers({InOperation.CloneModifier}, InModifierStack, CloneOp);

	if (!FailReason.IsEmpty())
	{
		UE_LOGF(LogActorModifierCoreLibrary, Warning, "CloneModifier %ls failing reason : %ls", *CloneOp.CloneModifier->GetModifierName().ToString(), *FailReason.ToString())
	}

	if (NewModifiers.IsEmpty())
	{
		return false;
	}

	OutNewModifier = NewModifiers.Last();

	return !!OutNewModifier;
}

bool UActorModifierCoreLibrary::MoveModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreMoveOperation& InOperation)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InModifierStack) || !IsValid(InModifierStack->GetModifiedActor()) || !ModifierSubsystem)
	{
		return false;
	}

	FText FailReason = FText::GetEmpty();
	FActorModifierCoreStackMoveOp MoveOp;
	MoveOp.MoveModifier = InOperation.MoveModifier;
	MoveOp.MovePosition = InOperation.MovePosition;
	MoveOp.MovePositionContext = InOperation.MovePositionContext;
	MoveOp.FailReason = &FailReason;

	if (!IsValid(MoveOp.MoveModifier))
	{
		return false;
	}

	const bool bModifierMoved = ModifierSubsystem->MoveModifier(InModifierStack, MoveOp);

	if (!FailReason.IsEmpty())
	{
		UE_LOGF(LogActorModifierCoreLibrary, Warning, "MoveModifier %ls failing reason : %ls", *MoveOp.MoveModifier->GetModifierName().ToString(), *FailReason.ToString())
	}

	return bModifierMoved;
}

bool UActorModifierCoreLibrary::RemoveModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreRemoveOperation& InOperation)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InModifierStack) || !IsValid(InModifierStack->GetModifiedActor()) || !ModifierSubsystem)
	{
		return false;
	}

	FText FailReason = FText::GetEmpty();
	FActorModifierCoreStackRemoveOp RemoveOp;
	RemoveOp.RemoveModifier = InOperation.RemoveModifier;
	RemoveOp.bRemoveDependencies = InOperation.bRemoveDependencies;
	RemoveOp.FailReason = &FailReason;

	if (!IsValid(RemoveOp.RemoveModifier))
	{
		return false;
	}

	const bool bModifierRemoved = ModifierSubsystem->RemoveModifiers({InOperation.RemoveModifier}, RemoveOp);

	if (!FailReason.IsEmpty())
	{
		UE_LOGF(LogActorModifierCoreLibrary, Warning, "RemoveModifier %ls failing reason : %ls", *RemoveOp.RemoveModifier->GetModifierName().ToString(), *FailReason.ToString())
	}

	return bModifierRemoved;
}

bool UActorModifierCoreLibrary::EnableModifier(UActorModifierCoreBase* InModifier, bool bInState)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(InModifier) || !IsValid(InModifier->GetModifiedActor()) || !ModifierSubsystem)
	{
		return false;
	}

	return ModifierSubsystem->EnableModifiers({InModifier}, bInState);
}

bool UActorModifierCoreLibrary::IsModifierEnabled(const UActorModifierCoreBase* InModifier, bool& bOutEnabled)
{
	if (!IsValid(InModifier) || !IsValid(InModifier->GetModifiedActor()))
	{
		return false;
	}

	bOutEnabled = InModifier->IsModifierEnabled();

	return true;
}

bool UActorModifierCoreLibrary::GetModifierStack(const UActorModifierCoreBase* InModifier, UActorModifierCoreStack*& OutModifierStack)
{
	OutModifierStack = nullptr;

	if (!IsValid(InModifier))
	{
		return false;
	}

	OutModifierStack = InModifier->GetModifierStack();

	return !!OutModifierStack;
}

bool UActorModifierCoreLibrary::GetModifierActor(const UActorModifierCoreBase* InModifier, AActor*& OutModifiedActor)
{
	OutModifiedActor = nullptr;

	if (!IsValid(InModifier))
	{
		return false;
	}

	OutModifiedActor = InModifier->GetModifiedActor();

	return !!OutModifiedActor;
}

bool UActorModifierCoreLibrary::GetModifierName(const UActorModifierCoreBase* InModifier, FName& OutModifierName)
{
	OutModifierName = NAME_None;

	if (!IsValid(InModifier))
	{
		return false;
	}

	OutModifierName = InModifier->GetModifierName();

	return !OutModifierName.IsNone();
}

bool UActorModifierCoreLibrary::GetModifierNameByClass(TSubclassOf<UActorModifierCoreBase> InModifierClass, FName& OutModifierName)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UClass* ModifierClass = InModifierClass.Get();
	OutModifierName = NAME_None;

	if (!ModifierClass || !ModifierSubsystem)
	{
		return false;
	}

	OutModifierName = ModifierSubsystem->GetRegisteredModifierName(ModifierClass);

	return !OutModifierName.IsNone();
}

bool UActorModifierCoreLibrary::GetModifierCategory(const UActorModifierCoreBase* InModifier, FName& OutModifierCategory)
{
	OutModifierCategory = NAME_None;

	if (!IsValid(InModifier))
	{
		return false;
	}

	OutModifierCategory = InModifier->GetModifierCategory();

	return !OutModifierCategory.IsNone();
}

bool UActorModifierCoreLibrary::GetModifierCategoryByClass(TSubclassOf<UActorModifierCoreBase> InModifierClass, FName& OutModifierCategory)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UClass* ModifierClass = InModifierClass.Get();
	OutModifierCategory = NAME_None;

	if (!ModifierClass || !ModifierSubsystem)
	{
		return false;
	}

	const FName ModifierName = ModifierSubsystem->GetRegisteredModifierName(ModifierClass);
	OutModifierCategory = ModifierSubsystem->GetModifierCategory(ModifierName);

	return !OutModifierCategory.IsNone();
}

bool UActorModifierCoreLibrary::GetModifierCategories(TSet<FName>& OutModifierCategories)
{
	OutModifierCategories.Empty();
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierSubsystem)
	{
		return false;
	}

	OutModifierCategories = ModifierSubsystem->GetModifierCategories();

	return true;
}

bool UActorModifierCoreLibrary::GetModifiersByCategory(FName InCategory, TSet<TSubclassOf<UActorModifierCoreBase>>& OutSupportedModifierClasses)
{
	OutSupportedModifierClasses.Empty();
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (InCategory.IsNone() || !ModifierSubsystem)
	{
		return false;
	}

	for (const FName& ModifierName : ModifierSubsystem->GetCategoryModifiers(InCategory))
	{
		TSubclassOf<UActorModifierCoreBase> ModifierClass = ModifierSubsystem->GetRegisteredModifierClass(ModifierName);

        if (ModifierClass.Get())
        {
			OutSupportedModifierClasses.Add(ModifierClass);
        }
	}

	return true;
}

bool UActorModifierCoreLibrary::GetModifierClass(FName InModifierName, TSubclassOf<UActorModifierCoreBase>& OutModifierClass)
{
	OutModifierClass = nullptr;
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (InModifierName.IsNone() || !ModifierSubsystem)
	{
		return false;
	}

	OutModifierClass = ModifierSubsystem->GetRegisteredModifierClass(InModifierName);

	return !!OutModifierClass.Get();
}

bool UActorModifierCoreLibrary::GetStackModifiers(const UActorModifierCoreStack* InModifierStack, TArray<UActorModifierCoreBase*>& OutModifiers)
{
	OutModifiers.Empty();

	if (!IsValid(InModifierStack) || !IsValid(InModifierStack->GetModifiedActor()))
	{
		return false;
	}

	OutModifiers = InModifierStack->GetModifiers();

	return true;
}

bool UActorModifierCoreLibrary::GetDependentModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutModifiers)
{
	OutModifiers.Empty();

	if (!IsValid(InModifier) || !InModifier->GetModifierStack())
	{
		return false;
	}

	const UActorModifierCoreStack* Stack = InModifier->GetModifierStack();

	Stack->GetDependentModifiers(InModifier, OutModifiers);

	return true;
}

bool UActorModifierCoreLibrary::GetRequiredModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutModifiers)
{
	OutModifiers.Empty();

	if (!IsValid(InModifier) || !InModifier->GetModifierStack())
	{
		return false;
	}

	const UActorModifierCoreStack* Stack = InModifier->GetModifierStack();

	Stack->GetRequiredModifiers(InModifier, OutModifiers);

	return true;
}

UActorModifierCoreBase* UActorModifierCoreLibrary::FindModifierByClass(UActorModifierCoreStack* InModifierStack, TSubclassOf<UActorModifierCoreBase> InModifierClass)
{
	if (IsValid(InModifierStack))
	{
		return InModifierStack->FindModifier(InModifierClass);
	}

	return nullptr;
}

UActorModifierCoreBase* UActorModifierCoreLibrary::FindModifierByName(const UActorModifierCoreStack* InModifierStack, FName InModifierName)
{
	if (IsValid(InModifierStack))
	{
		return InModifierStack->FindModifier(InModifierName);
	}

	return nullptr;
}

TArray<UActorModifierCoreBase*> UActorModifierCoreLibrary::FindModifiersByClass(UActorModifierCoreStack* InModifierStack, TSubclassOf<UActorModifierCoreBase> InModifierClass)
{
	if (IsValid(InModifierStack))
	{
		return InModifierStack->FindModifiers(InModifierClass);
	}

	return {};
}

TArray<UActorModifierCoreBase*> UActorModifierCoreLibrary::FindModifiersByName(UActorModifierCoreStack* InModifierStack, FName InModifierName)
{
	if (IsValid(InModifierStack))
	{
		return InModifierStack->FindModifiers(InModifierName);
	}

	return {};
}

bool UActorModifierCoreLibrary::ContainsModifier(UActorModifierCoreStack* InModifierStack, UActorModifierCoreBase* InModifier)
{
	if (IsValid(InModifierStack))
	{
		return InModifierStack->ContainsModifier(InModifier);
	}

	return false;
}

bool UActorModifierCoreLibrary::GetSupportedModifiers(AActor* InActor, TSet<TSubclassOf<UActorModifierCoreBase>>& OutSupportedModifierClasses, EActorModifierCoreStackPosition InContextPosition, UActorModifierCoreBase* InContextModifier)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	OutSupportedModifierClasses.Empty();

	if (IsValid(InActor) || !ModifierSubsystem)
	{
		return false;
	}

	for (const FName& ModifierName : ModifierSubsystem->GetAllowedModifiers(InActor, InContextModifier, InContextPosition))
	{
		TSubclassOf<UActorModifierCoreBase> ModifierClass = ModifierSubsystem->GetRegisteredModifierClass(ModifierName);

		if (ModifierClass.Get())
		{
			OutSupportedModifierClasses.Add(ModifierClass);
		}
	}

	return true;
}

bool UActorModifierCoreLibrary::GetAvailableModifiers(TSet<TSubclassOf<UActorModifierCoreBase>>& OutAvailableModifierClasses)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	OutAvailableModifierClasses.Empty();

	if (!ModifierSubsystem)
	{
		return false;
	}

	OutAvailableModifierClasses = ModifierSubsystem->GetRegisteredModifierClasses();

	return true;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataName(FActorModifierCoreMetadata& InMetadata, FName InName)
{
	InMetadata.SetName(InName);
	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataCategory(FActorModifierCoreMetadata& InMetadata, FName InCategory)
{
	InMetadata.SetCategory(InCategory);
	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataDisplayName(FActorModifierCoreMetadata& InMetadata, const FText& InName)
{
#if WITH_EDITOR
	InMetadata.SetDisplayName(InName);
#endif
	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataColor(FActorModifierCoreMetadata& InMetadata, const FLinearColor& InColor)
{
#if WITH_EDITOR
	InMetadata.SetColor(InColor);
#endif
	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataDescription(FActorModifierCoreMetadata& InMetadata, const FText& InDescription)
{
#if WITH_EDITOR
	InMetadata.SetDescription(InDescription);
#endif
	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::AddModifierMetadataDependency(FActorModifierCoreMetadata& InMetadata, TSubclassOf<UActorModifierCoreBase> InModifierClass)
{
	if (const UActorModifierCoreBase* Modifier = InModifierClass.GetDefaultObject())
	{
		InMetadata.AddDependency(Modifier->GetModifierName());
	}

	return InMetadata;
}

FActorModifierCoreMetadata& UActorModifierCoreLibrary::SetModifierMetadataCompatibilityRule(FActorModifierCoreMetadata& InMetadata, const FModifierCompatibilityRule& InDelegate)
{
	InMetadata.SetCompatibilityRule(InDelegate);
	return InMetadata;
}
