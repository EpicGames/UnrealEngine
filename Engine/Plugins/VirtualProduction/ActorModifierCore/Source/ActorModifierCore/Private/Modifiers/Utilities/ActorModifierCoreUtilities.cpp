// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"

#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/ActorModifierCoreStack.h"

namespace UE::ActorModifierCore::Utilities
{

UActorModifierCoreBase* FindFirstActorModifierByClass(const AActor* InStartActor, const TSubclassOf<UActorModifierCoreBase>& InModifierClass)
{
	UActorModifierCoreBase* FoundModifier = nullptr;

	if (!InStartActor || !InModifierClass)
	{
		return FoundModifier;
	}

	if (const UActorModifierCoreComponent* ModifierComponent = InStartActor->FindComponentByClass<UActorModifierCoreComponent>())
	{
		if (const UActorModifierCoreStack* ModifierStack = ModifierComponent->GetModifierStack())
		{
			FoundModifier = ModifierStack->FindModifier(InModifierClass.Get());
		}
	}

	if (FoundModifier)
	{
		return FoundModifier;
	}

	return FindFirstActorModifierByClass(InStartActor->GetAttachParentActor(), InModifierClass);
}

FString GetActorNameSafe(const AActor* InActor)
{
	if (InActor)
	{
		return InActor->GetActorNameOrLabel();
	}
	return TEXT("None");
}

} // UE::ActorModifierCore::Utilities
