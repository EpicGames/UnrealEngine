// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/ActorModifierCoreSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Text.h"
#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/ActorModifierCoreSharedActor.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "ActorModifierCoreSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreSubsystem, Log, All);

UActorModifierCoreSubsystem::FOnModifierClassRegistered UActorModifierCoreSubsystem::OnModifierClassRegisteredDelegate;
UActorModifierCoreSubsystem::FOnModifierClassRegistered UActorModifierCoreSubsystem::OnModifierClassUnregisteredDelegate;
UActorModifierCoreSubsystem::FOnModifierStackRegistered UActorModifierCoreSubsystem::OnModifierStackRegisteredDelegate;
UActorModifierCoreSubsystem::FOnModifierStackRegistered UActorModifierCoreSubsystem::OnModifierStackUnregisteredDelegate;
UActorModifierCoreSubsystem::FOnModifierReplaced UActorModifierCoreSubsystem::OnModifierReplacedDelegate;
FTSTicker UActorModifierCoreSubsystem::ModifierTicker;

UActorModifierCoreSubsystem::UActorModifierCoreSubsystem()
	: UEngineSubsystem()
{
}

UActorModifierCoreSubsystem* UActorModifierCoreSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UActorModifierCoreSubsystem>();
	}
	return nullptr;
}

FTSTicker& UActorModifierCoreSubsystem::GetModifierTicker()
{
	return ModifierTicker;
}

void UActorModifierCoreSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	ScanForModifiers();

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.OnFilesLoaded().AddUObject(this, &UActorModifierCoreSubsystem::OnAssetRegistryFilesLoaded);
	AssetRegistry.OnAssetAdded().AddUObject(this, &UActorModifierCoreSubsystem::OnAssetRegistryAssetAdded);
	AssetRegistry.OnAssetRemoved().AddUObject(this, &UActorModifierCoreSubsystem::OnAssetRegistryAssetRemoved);
	AssetRegistry.OnAssetUpdated().AddUObject(this, &UActorModifierCoreSubsystem::OnAssetRegistryAssetUpdated);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UActorModifierCoreSubsystem::OnBlueprintObjectsReplaced);
#endif
}

void UActorModifierCoreSubsystem::Deinitialize()
{
	ModifierTicker.Reset();

	// create a copy to avoid issue with iterator and removal
	TArray<FName> ModifierNames;
	ModifiersMetadata.GetKeys(ModifierNames);
	for (FName& ModifierName : ModifierNames)
	{
		UnregisterModifierClass(ModifierName);
	}

	if (const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry& AssetRegistry = AssetRegistryModule->Get();
		AssetRegistry.OnFilesLoaded().RemoveAll(this);
		AssetRegistry.OnAssetAdded().RemoveAll(this);
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
		AssetRegistry.OnAssetUpdated().RemoveAll(this);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif

	Super::Deinitialize();
}

void UActorModifierCoreSubsystem::OnInsertModifier(const FActorModifierCoreStackInsertOp& InInsertOp) const
{
#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("Action"), TEXT("Insert"));
		Attributes.Emplace(TEXT("Name"), InInsertOp.NewModifierName.ToString());
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ActorModifiers.Modifier"), Attributes);
	}
#endif
}

bool UActorModifierCoreSubsystem::RegisterModifierClass(const UClass* InModifierClass, bool bInOverrideIfExists)
{
	if (!InModifierClass)
	{
		return false;
	}

	if (!InModifierClass->IsChildOf(UActorModifierCoreBase::StaticClass())
		|| InModifierClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (!bInOverrideIfExists && IsRegisteredModifierClass(InModifierClass))
	{
		return false;
	}

	if (UActorModifierCoreBase* CDO = InModifierClass->GetDefaultObject<UActorModifierCoreBase>())
	{
		CDO->PostModifierCDOCreation();

		if (!CDO->Metadata.IsValid() || !CDO->Metadata->GetClass())
		{
			UE_LOGF(LogActorModifierCoreSubsystem, Warning, "[%ls] Could not register modifier class : Modifier metadata is invalid", *CDO->GetClass()->GetName())
			return false;	
		}
		
		if (CDO->IsModifierStack())
		{
			return false;
		}

		const FName ModifierName = CDO->GetModifierName();

		if (ModifierName.IsNone())
		{
			UE_LOGF(LogActorModifierCoreSubsystem, Warning, "[%ls] Could not register modifier class : Modifier name is invalid", *CDO->GetClass()->GetName())
			return false;
		}

		if (bInOverrideIfExists && IsRegisteredModifierClass(ModifierName))
		{
			UnregisterModifierClass(ModifierName);
		}

		ModifiersMetadata.Add(ModifierName, CDO->Metadata.ToSharedRef());

		UActorModifierCoreSubsystem::OnModifierClassRegisteredDelegate.Broadcast(*CDO->Metadata);

		return true;
	}

	return false;
}

bool UActorModifierCoreSubsystem::UnregisterModifierClass(const FName& InName)
{
	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InName))
	{
		UActorModifierCoreSubsystem::OnModifierClassUnregisteredDelegate.Broadcast(**ModifierMetadata);
		return ModifiersMetadata.Remove(InName) != 0;
	}

	return false;
}

bool UActorModifierCoreSubsystem::UnregisterModifierClass(const UClass* InModifierClass)
{
	bool bUnregistered = false;

	for (TMap<FName, TSharedRef<FActorModifierCoreMetadata>>::TIterator It(ModifiersMetadata); It; ++It)
	{
		if (It->Value->GetClass() == InModifierClass)
		{
			UActorModifierCoreSubsystem::OnModifierClassUnregisteredDelegate.Broadcast(*It.Value());
			It.RemoveCurrent();
			bUnregistered = true;
		}
	}

	return bUnregistered;
}

bool UActorModifierCoreSubsystem::IsRegisteredModifierClass(const FName& InName) const
{
	return ModifiersMetadata.Contains(InName);
}

bool UActorModifierCoreSubsystem::IsRegisteredModifierClass(const UClass* InClass) const
{
	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (ModifierMetadataPair.Value->GetClass() == InClass)
		{
			return true;
		}
	}

	return false;
}

FName UActorModifierCoreSubsystem::GetRegisteredModifierName(const UClass* InModifierClass) const
{
	if (!InModifierClass)
	{
		return NAME_None;
	}

	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (ModifierMetadataPair.Value->GetClass() == InModifierClass)
		{
			return ModifierMetadataPair.Key;
		}
	}

	return NAME_None;
}

TSubclassOf<UActorModifierCoreBase> UActorModifierCoreSubsystem::GetRegisteredModifierClass(FName InModifierName) const
{
	TSubclassOf<UActorModifierCoreBase> ModifierClass;

	if (InModifierName.IsNone())
	{
		return ModifierClass;
	}

	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InModifierName))
	{
		ModifierClass = (*ModifierMetadata)->GetClass();
	}

	return ModifierClass;
}

TSet<TSubclassOf<UActorModifierCoreBase>> UActorModifierCoreSubsystem::GetRegisteredModifierClasses() const
{
	TSet<TSubclassOf<UActorModifierCoreBase>> ModifiersClass;
	ModifiersClass.Reserve(ModifiersMetadata.Num());

	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (ModifierMetadataPair.Value->GetClass())
		{
			ModifiersClass.Add(ModifierMetadataPair.Value->GetClass());
		}
	}

	return ModifiersClass;
}

TSet<FName> UActorModifierCoreSubsystem::GetRegisteredModifiers() const
{
	TArray<FName> OutModifiers;
	ModifiersMetadata.GenerateKeyArray(OutModifiers);
	return TSet<FName>(OutModifiers);
}

TSet<FName> UActorModifierCoreSubsystem::GetAllowedModifiers(AActor* InActor, UActorModifierCoreBase* InContextModifier, EActorModifierCoreStackPosition InContextPosition) const
{
	TSet<FName> OutModifiers;

	if (!IsValid(InActor))
	{
		return OutModifiers;
	}

	if (IsValid(InContextModifier) && InContextModifier->IsModifierStack())
	{
		return OutModifiers;
	}

	// get actor supported modifiers
	ForEachModifierMetadata([InActor, &OutModifiers](const FActorModifierCoreMetadata& InMetadata)->bool
	{
		if (InMetadata.IsCompatibleWith(InActor))
		{
			OutModifiers.Add(InMetadata.GetName());
		}
		return true;
	});

	const UActorModifierCoreStack* ActorRootStack = GetActorModifierStack(InActor);

	if (!IsValid(ActorRootStack))
	{
		return OutModifiers;
	}

	UActorModifierCoreBase* BeforeModifier = InContextPosition == EActorModifierCoreStackPosition::Before ? InContextModifier : nullptr;
	UActorModifierCoreBase* AfterModifier = InContextPosition == EActorModifierCoreStackPosition::After ? nullptr : InContextModifier;

	// filter modifiers based on modifiers already inside the stack
	bool bPrecedingContextModifier = true;
	ActorRootStack->ProcessFunction([this, BeforeModifier, AfterModifier, &OutModifiers, &bPrecedingContextModifier](const UActorModifierCoreBase* InModifier)->bool
	{
		TSet<FName> RemoveModifiers;

		if (BeforeModifier && BeforeModifier == InModifier)
		{
			bPrecedingContextModifier = false;
		}

		for (const FName& SupportedModifier : OutModifiers)
		{
			ProcessModifierMetadata(SupportedModifier, [&RemoveModifiers, bPrecedingContextModifier, InModifier](const FActorModifierCoreMetadata& InMetadata)->bool
			{
				// remove multiple modifier if not allowed
				if (!InMetadata.IsMultipleAllowed() && InModifier->GetModifierName() == InMetadata.GetName())
				{
					RemoveModifiers.Add(InMetadata.GetName());
				}

				// remove modifier if not allowed before or after this one
				if (bPrecedingContextModifier)
				{
					// check if we can add this modifier after this since we are preceding context modifier
					if (!InMetadata.IsAllowedAfter(InModifier->GetModifierName()))
					{
						RemoveModifiers.Add(InMetadata.GetName());
					}
				}
				else
				{
					//  check if we can add this modifier before this since we are following context modifier
					if (!InMetadata.IsAllowedBefore(InModifier->GetModifierName()))
					{
						RemoveModifiers.Add(InMetadata.GetName());
					}
				}

				return true;
			});
		}

		// supported modifiers minus remove modifiers
		OutModifiers = OutModifiers.Difference(RemoveModifiers);

		// leave loop if empty
		if (OutModifiers.IsEmpty())
		{
			return false;
		}

		if (AfterModifier && AfterModifier == InModifier)
		{
			bPrecedingContextModifier = false;
		}

		return true;
	});

	return OutModifiers;
}

TSet<FName> UActorModifierCoreSubsystem::GetCategoryModifiers(const FName& InCategory) const
{
	TSet<FName> OutModifiers;
	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (InCategory == ModifierMetadataPair.Value->GetCategory())
		{
			OutModifiers.Add(ModifierMetadataPair.Key);
		}
	}
	return OutModifiers;
}

TSet<FName> UActorModifierCoreSubsystem::GetModifierCategories() const
{
	TSet<FName> OutCategories;
	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		OutCategories.Add(ModifierMetadataPair.Value->GetCategory());
	}
	return OutCategories;
}

FName UActorModifierCoreSubsystem::GetModifierCategory(const FName& InModifier) const
{
	FName Category;
	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InModifier))
	{
		Category = (*ModifierMetadata)->GetCategory();
	}
	return Category;
}

#if WITH_EDITOR
TSet<FName> UActorModifierCoreSubsystem::GetHiddenModifiers() const
{
	TSet<FName> OutModifiers;
	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (ModifierMetadataPair.Value->IsHidden())
		{
			OutModifiers.Add(ModifierMetadataPair.Value->GetName());
		}
	}
	return OutModifiers;
}
#endif

TArray<UActorModifierCoreBase*> UActorModifierCoreSubsystem::GetAllowedMoveModifiers(UActorModifierCoreBase* InMoveModifier) const
{
	TArray<UActorModifierCoreBase*> AllowedModifiers;

	if (!IsValid(InMoveModifier))
	{
		return AllowedModifiers;
	}

	const UActorModifierCoreStack* ModifierStack = InMoveModifier->GetModifierStack();

	const TConstArrayView<UActorModifierCoreBase*> StackModifiers = ModifierStack->GetModifiers();

	if (!StackModifiers.Contains(InMoveModifier))
	{
		return AllowedModifiers;
	}

	TSet<UActorModifierCoreBase*> OutRequiredModifiers;
	ModifierStack->GetRequiredModifiers(InMoveModifier, OutRequiredModifiers);

	TSet<UActorModifierCoreBase*> OutDependentModifiers;
	ModifierStack->GetDependentModifiers(InMoveModifier, OutDependentModifiers);

	ModifierStack->GetRootModifierStack()->ProcessFunction([this, &OutRequiredModifiers, &OutDependentModifiers, &AllowedModifiers, InMoveModifier](const UActorModifierCoreBase* InModifier)->bool
	{
		if (OutRequiredModifiers.Contains(InModifier))
		{
			OutRequiredModifiers.Remove(InModifier);
		}
		else if (OutDependentModifiers.Contains(InModifier))
		{
			return false;
		}
		else if (OutRequiredModifiers.IsEmpty())
		{
			if (InModifier != InMoveModifier)
			{
				AllowedModifiers.Add(const_cast<UActorModifierCoreBase*>(InModifier));
			}
		}

		return true;
	});

	return AllowedModifiers;
}

void UActorModifierCoreSubsystem::GetSortedModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, AActor* InTargetActor, UActorModifierCoreBase* InTargetModifier, EActorModifierCoreStackPosition InPosition, TArray<UActorModifierCoreBase*>& OutMoveModifiers, TArray<UActorModifierCoreBase*>& OutCloneModifiers) const
{
	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem) || InModifiers.IsEmpty() || !IsValid(InTargetActor))
	{
		return;
	}

	for (UActorModifierCoreBase* Modifier : InModifiers)
	{
		if (!IsValid(Modifier))
		{
			continue;
		}

		// it's a clone operation if target actor is different as modifier actor
		if (Modifier->GetModifiedActor() != InTargetActor)
		{
			OutCloneModifiers.Add(Modifier);
		}
		// it's a move operation if target actor is same as modifier actor
		else
		{
			OutMoveModifiers.Add(Modifier);
		}
	}

	// Remove unsupported modifiers when inserting where target modifier is nullptr
	if (!InTargetModifier)
	{
		const TSet<FName> AllowedModifiers = ModifierSubsystem->GetAllowedModifiers(InTargetActor, InTargetModifier, InPosition);

		OutMoveModifiers.RemoveAll([&AllowedModifiers](UActorModifierCoreBase* InModifier)
		{
			return !IsValid(InModifier) || !AllowedModifiers.Contains(InModifier->GetModifierName());
		});

		OutCloneModifiers.RemoveAll([&AllowedModifiers](UActorModifierCoreBase* InModifier)
		{
			return !IsValid(InModifier) || !AllowedModifiers.Contains(InModifier->GetModifierName());
		});
	}

	// Sort them by dependency and current order
	OutMoveModifiers.Sort([InTargetModifier](const UActorModifierCoreBase& A, const UActorModifierCoreBase& B)
	{
		const FActorModifierCoreMetadata& MetadataA = A.GetModifierMetadata();
		const FActorModifierCoreMetadata& MetadataB = B.GetModifierMetadata();
		const bool bADependsOnB = MetadataA.DependsOn(MetadataB.GetName());

		const UActorModifierCoreStack* StackA = A.GetModifierStack();
		const UActorModifierCoreStack* StackB = B.GetModifierStack();
		const bool bAIsBeforeB = StackA == StackB && StackA->ContainsModifierBefore(&A, &B);

		bool bSortOrder = bAIsBeforeB && !bADependsOnB;

		const UActorModifierCoreBase* DependencyModifier = bADependsOnB ? &A : &B;
		if (StackA->ContainsModifierAfter(InTargetModifier, DependencyModifier))
		{
			bSortOrder = !bSortOrder;
		}

		return bSortOrder;
	});

	OutCloneModifiers.Sort([](const UActorModifierCoreBase& A, const UActorModifierCoreBase& B)
	{
		const FActorModifierCoreMetadata& MetadataA = A.GetModifierMetadata();
		const FActorModifierCoreMetadata& MetadataB = B.GetModifierMetadata();
		const bool bADependsOnB = MetadataA.DependsOn(MetadataB.GetName());

		const UActorModifierCoreStack* StackA = A.GetModifierStack();
		const UActorModifierCoreStack* StackB = B.GetModifierStack();
		const bool bAIsBeforeB = StackA == StackB && StackA->ContainsModifierBefore(&A, &B);

		const bool bSortOrder = bAIsBeforeB && !bADependsOnB;

		return bSortOrder;
	});
}

ETickableTickType UActorModifierCoreSubsystem::GetTickableTickType() const
{
	return ETickableTickType::Always;
}

bool UActorModifierCoreSubsystem::IsTickableInEditor() const
{
	return true;
}

void UActorModifierCoreSubsystem::Tick(float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorModifierCoreSubsystem::Tick);
	ModifierTicker.Tick(InDeltaTime);
}

TStatId UActorModifierCoreSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UActorModifierCoreSubsystem, STATGROUP_Tickables);
}

bool UActorModifierCoreSubsystem::MoveModifiers(const TArray<UActorModifierCoreBase*>& InModifiers, UActorModifierCoreStack* InStack, FActorModifierCoreStackMoveOp& InMoveOp) const
{
	if (!IsValid(InStack) || InModifiers.IsEmpty())
	{
		return false;
	}

#if WITH_EDITOR
	static const FText TransactionText = LOCTEXT("MoveModifiers", "Moving {0} modifier(s) {1} modifier {2}");
	FScopedTransaction Transaction(
		FText::Format(TransactionText
			, FText::FromString(FString::FromInt(InModifiers.Num()))
			, FText::FromString(InMoveOp.MovePosition == EActorModifierCoreStackPosition::After ? TEXT("after") : TEXT("before"))
			, FText::FromName(InMoveOp.MovePositionContext ? InMoveOp.MovePositionContext->GetModifierName() : InStack->GetModifierName()))
		, InMoveOp.bShouldTransact);
#endif

	uint32 EditModifierCount = 0;
	UActorModifierCoreBase* TargetModifier = InMoveOp.MovePositionContext;

	InStack->ProcessLockFunction([InStack, TargetModifier, &InModifiers, &InMoveOp, &EditModifierCount]()
	{
		UActorModifierCoreBase* OperationContext = nullptr;

		for (UActorModifierCoreBase* Modifier : InModifiers)
		{
			if (IsValid(Modifier))
			{
				InMoveOp.MoveModifier = Modifier;
				InMoveOp.MovePositionContext = TargetModifier;

				// Eg 1: when moving [bend, subdivide] before target modifier X : X is after them in the stack, we want bend added before X and subdivide added before Bend
				// Eg 2: when moving [subdivide, bend] after target modifier X : X is before them in the stack, we want subdivide after X and bend after subdivide
				if (OperationContext &&
					((InMoveOp.MovePosition == EActorModifierCoreStackPosition::After && InStack->ContainsModifierAfter(Modifier, TargetModifier)) ||
					(InMoveOp.MovePosition == EActorModifierCoreStackPosition::Before && InStack->ContainsModifierBefore(Modifier, TargetModifier))))
				{
					InMoveOp.MovePositionContext = OperationContext;
				}

				if (InStack->MoveModifier(InMoveOp))
				{
					++EditModifierCount;
					OperationContext = Modifier;
				}
				else
				{
					const AActor* TargetActor = TargetModifier->GetModifiedActor();
					const FText& ErrorText = InMoveOp.FailReason ? *InMoveOp.FailReason : FText::GetEmpty();

					// Move modifiers on actor failed
					UE_LOGF(LogActorModifierCoreSubsystem, Warning, "Move modifier %ls on actor %ls failed : %ls",
						*InMoveOp.MoveModifier->GetModifierName().ToString(),
						*TargetActor->GetActorNameOrLabel(),
						*ErrorText.ToString());

					break;
				}
			}
		}
	});

	return EditModifierCount > 0;
}

TArray<UActorModifierCoreBase*> UActorModifierCoreSubsystem::CloneModifiers(const TArray<UActorModifierCoreBase*>& InModifiers, UActorModifierCoreStack* InStack, FActorModifierCoreStackCloneOp& InCloneOp) const
{
	TArray<UActorModifierCoreBase*> NewModifiers;

	if (!IsValid(InStack) || InModifiers.IsEmpty())
	{
		return NewModifiers;
	}

#if WITH_EDITOR
	static const FText TransactionText = LOCTEXT("CloneModifiers", "Cloning {0} modifier(s) {1} modifier {2}");
	FScopedTransaction Transaction(
		FText::Format(TransactionText
			, FText::FromString(FString::FromInt(InModifiers.Num()))
			, FText::FromString(InCloneOp.ClonePosition == EActorModifierCoreStackPosition::After ? TEXT("after") : TEXT("before"))
			, FText::FromName(InCloneOp.ClonePositionContext ? InCloneOp.ClonePositionContext->GetModifierName() : InStack->GetModifierName()))
		, InCloneOp.bShouldTransact);
#endif

	uint32 EditModifierCount = 0;
	UActorModifierCoreBase* TargetModifier = InCloneOp.ClonePositionContext;

	InStack->ProcessLockFunction([InStack, TargetModifier, &InModifiers, &InCloneOp, &EditModifierCount, &NewModifiers]()
	{
		UActorModifierCoreBase* OperationContext = nullptr;
		for (UActorModifierCoreBase* Modifier : InModifiers)
		{
			if (IsValid(Modifier))
			{
				InCloneOp.ClonePositionContext = TargetModifier;
				InCloneOp.CloneModifier = Modifier;

				// When inserting A, B where B depends on A after C, clone A after C then clone B after A
				if (OperationContext && InCloneOp.ClonePosition == EActorModifierCoreStackPosition::After)
				{
					InCloneOp.ClonePositionContext = OperationContext;
				}

				UActorModifierCoreBase* NewClonedModifier = InStack->CloneModifier(InCloneOp);
				NewModifiers.Add(NewClonedModifier);

				if (NewClonedModifier)
				{
					++EditModifierCount;
					OperationContext = NewClonedModifier;
				}
				else
				{
					const AActor* TargetActor = TargetModifier->GetModifiedActor();
					const FText& ErrorText = InCloneOp.FailReason ? *InCloneOp.FailReason : FText::GetEmpty();

					// Clone modifiers on actor failed
					UE_LOGF(LogActorModifierCoreSubsystem, Warning, "Clone modifier %ls on actor %ls failed : %ls",
						*InCloneOp.CloneModifier->GetModifierName().ToString(),
						*TargetActor->GetActorNameOrLabel(),
						*ErrorText.ToString());

					break;
				}
			}
		}
	});

	return NewModifiers;
}

bool UActorModifierCoreSubsystem::ValidateModifierCreation(const FName& InName, const UActorModifierCoreStack* InStack, FText& OutFailReason, UActorModifierCoreBase* InBeforeModifier) const
{
	if (!IsValid(InStack))
	{
		OutFailReason = LOCTEXT("InvalidStackProvided", "Modifier stack provided is invalid");
		return false;
	}

	const AActor* ModifiedActor = InStack->GetModifiedActor();
	if (!IsValid(ModifiedActor))
	{
		OutFailReason = LOCTEXT("InvalidActorProvided", "Modified stack actor is invalid");
		return false;
	}

	// check modifier is registered
	TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadataPtr = ModifiersMetadata.Find(InName);
	if (!ModifierMetadataPtr)
	{
		OutFailReason = FText::Format(LOCTEXT("NoModifierClassRegistered", "No modifier class registered for this modifier name : {0}"), FText::FromName(InName));
		return false;
	}

	const TSharedRef<FActorModifierCoreMetadata> ModifierMetadata = *ModifierMetadataPtr;

	// check actor is supported by modifier
	if (!ModifierMetadata->IsCompatibleWith(ModifiedActor))
	{
		OutFailReason = FText::Format(LOCTEXT("ModifierActorNotSupported", "Modifier {0} does not support this actor {1}"), FText::FromName(InName), FText::FromString(ModifiedActor->GetActorNameOrLabel()));
		return false;
	}

	// 1. check dependencies, 2. if multiple modifier are allowed, 3. if modifier are allowed before and after
	TArray<FName> ModifierDependencies(ModifierMetadata->GetDependencies());
	// if before modifier is invalid we add the modifier at the end of the stack
	bool bCurrentModifierBefore = true;
	const bool bMultipleAllowed = ModifierMetadata->IsMultipleAllowed();
	const bool bResult =
		InStack->GetRootModifierStack()->ProcessFunction([InName, bMultipleAllowed, ModifierMetadata, &OutFailReason, InBeforeModifier, &bCurrentModifierBefore, &ModifierDependencies](const UActorModifierCoreBase* InModifier)->bool{

			// check if we are after or before the last modifier
			if (InBeforeModifier && InModifier == InBeforeModifier)
			{
				bCurrentModifierBefore = false;
			}

			// are we creating new modifier after this one ?
			if (bCurrentModifierBefore)
			{
				ModifierDependencies.Remove(InModifier->GetModifierName());

				// check if current modifier is allowed after new modifier we want to add
				if (!ModifierMetadata->IsAllowedAfter(InModifier->GetModifierName()))
				{
					OutFailReason = FText::Format(LOCTEXT("NoModifierAllowedAfter", "Modifier {0} is not allowed after modifier {1}"), FText::FromName(InName), FText::FromName(InModifier->GetModifierName()));
					return false;
				}
			}
			else
			{
				// check if current modifier is allowed before new modifier we want to add
				if (!ModifierMetadata->IsAllowedBefore(InModifier->GetModifierName()))
				{
					OutFailReason = FText::Format(LOCTEXT("NoModifierAllowedBefore", "Modifier {0} is not allowed before modifier {1}"), FText::FromName(InName), FText::FromName(InModifier->GetModifierName()));
					return false;
				}
			}

			// check for multiple modifier of the same type
			if (!bMultipleAllowed)
			{
				if (InModifier->GetModifierName() == InName)
				{
					OutFailReason = FText::Format(LOCTEXT("NoMultipleModifierAllowed", "Modifier {0} already exists in stack, unallowed multiple modifier"), FText::FromName(InName));
					return false;
				}
			}

			return true;
		});

	if (!bResult)
	{
		return false;
	}

	// check missing dependencies modifiers
	for (FName& ModifierDependency : ModifierDependencies)
	{
		if (!ValidateModifierCreation(ModifierDependency, InStack, OutFailReason, InBeforeModifier))
		{
			OutFailReason = FText::Format(LOCTEXT("ModifierRequiredBefore", "Modifier {0} is a dependency for {1} : {2}"), FText::FromName(ModifierDependency), FText::FromName(InName), OutFailReason);
			return false;
		}
	}

	return true;
}

UActorModifierCoreBase* UActorModifierCoreSubsystem::CreateModifierInstance(const FName& InName, UActorModifierCoreStack* InStack, FText& OutFailReason, UActorModifierCoreBase* InBeforeModifier) const
{
	UActorModifierCoreBase* NewModifierInstance = nullptr;
	if (!ValidateModifierCreation(InName, InStack, OutFailReason, InBeforeModifier))
	{
		return NewModifierInstance;
	}

	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InName))
	{
		NewModifierInstance = (*ModifierMetadata)->CreateModifierInstance(InStack);
	}

	return NewModifierInstance;
}

bool UActorModifierCoreSubsystem::BuildModifierDependencies(const FName& InName, TArray<FName>& OutBeforeModifiers) const
{
	if (InName.IsNone())
	{
		return false;
	}

	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InName))
	{
		// get required modifier before this one is added
		const TConstArrayView<FName> RequiredModifiers = (*ModifierMetadata)->GetDependencies();

		// check recursively for required modifier
		for(const FName& RequiredModifier : RequiredModifiers)
		{
			// if we don't have the modifier in the stack, we need to add it before this one
			if (!OutBeforeModifiers.Contains(RequiredModifier))
			{
				if (!BuildModifierDependencies(RequiredModifier, OutBeforeModifiers))
				{
					return false;
				}
			}
		}

		// add as dependency in the end to avoid reversing the array
		OutBeforeModifiers.AddUnique(InName);

		return true;
	}

	return false;
}

bool UActorModifierCoreSubsystem::RegisterActorModifierStack(UActorModifierCoreStack* InStack)
{
	// only register root actor stack
	if (!InStack || !InStack->IsRootStack())
	{
		return false;
	}

	// is stack actor valid
	const AActor* Actor = InStack->GetModifiedActor();
	if (!Actor)
	{
		return false;
	}

	// is a stack already registered
	if (const TWeakObjectPtr<UActorModifierCoreStack>* ActorStack = ModifierStacks.Find(Actor))
	{
		// already root stack registered
		if ((*ActorStack).IsValid())
		{
			return false;
		}

		// remove invalid stack
		ModifierStacks.Remove(Actor);
	}

	// register new stack for actor
	ModifierStacks.Add(Actor, InStack);

	UActorModifierCoreSubsystem::OnModifierStackRegisteredDelegate.Broadcast(InStack);

	return true;
}

bool UActorModifierCoreSubsystem::UnregisterActorModifierStack(const AActor* InActor)
{
	if (!InActor)
	{
		return false;
	}

	// is a stack registered for this actor
	if (const TWeakObjectPtr<UActorModifierCoreStack>* ActorStack = ModifierStacks.Find(InActor))
	{
		if ((*ActorStack).IsValid())
		{
			UActorModifierCoreSubsystem::OnModifierStackUnregisteredDelegate.Broadcast((*ActorStack).Get());
		}

		ModifierStacks.Remove(InActor);

		return true;
	}
	return false;
}

bool UActorModifierCoreSubsystem::UnregisterActorModifierStack(const UActorModifierCoreStack* InStack)
{
	if (const AActor* Actor = GetModifierStackActor(InStack))
	{
		return UnregisterActorModifierStack(Actor);
	}
	return false;
}

bool UActorModifierCoreSubsystem::IsRegisteredActorModifierStack(const AActor* InActor) const
{
	if (GetActorModifierStack(InActor))
	{
		return true;
	}
	return false;
}

bool UActorModifierCoreSubsystem::IsRegisteredActorModifierStack(const UActorModifierCoreStack* InStack) const
{
	if (const UActorModifierCoreStack* ActorStack = GetActorModifierStack(InStack->GetModifiedActor()))
	{
		return ActorStack == InStack;
	}
	return false;
}

UActorModifierCoreStack* UActorModifierCoreSubsystem::GetActorModifierStack(const AActor* InActor) const
{
	if (const TWeakObjectPtr<UActorModifierCoreStack>* ActorStack = ModifierStacks.Find(InActor))
	{
		return ActorStack->Get();
	}
	return nullptr;
}

UActorModifierCoreStack* UActorModifierCoreSubsystem::AddActorModifierStack(AActor* InActor) const
{
	// is it already registered and attached to that actor
	if (const TWeakObjectPtr<UActorModifierCoreStack>* ActorStack = ModifierStacks.Find(InActor))
	{
		return (*ActorStack).Get();
	}

	// Create modifier component and return new stack
	if (const UActorModifierCoreComponent* ModifierComponent = UActorModifierCoreComponent::CreateAndExposeComponent(InActor))
	{
		return ModifierComponent->GetModifierStack();
	}

	return nullptr;
}

const AActor* UActorModifierCoreSubsystem::GetModifierStackActor(const UActorModifierCoreStack* InStack) const
{
	for (const TPair<TWeakObjectPtr<const AActor>, TWeakObjectPtr<UActorModifierCoreStack>>& ModifierStackPair : ModifierStacks)
	{
		if (ModifierStackPair.Value.Get() == InStack)
		{
			return ModifierStackPair.Key.Get();
		}
	}
	return nullptr;
}

bool UActorModifierCoreSubsystem::ForEachModifierMetadata(TFunctionRef<bool(const FActorModifierCoreMetadata&)> InProcessFunction) const
{
	for (const TPair<FName, TSharedRef<FActorModifierCoreMetadata>>& ModifierMetadataPair : ModifiersMetadata)
	{
		if (!InProcessFunction(*ModifierMetadataPair.Value))
		{
			return false;
		}
	}
	return true;
}

bool UActorModifierCoreSubsystem::ProcessModifierMetadata(const FName& InName, TFunctionRef<bool(const FActorModifierCoreMetadata&)> InProcessFunction) const
{
	if (TSharedRef<FActorModifierCoreMetadata> const* ModifierMetadata = ModifiersMetadata.Find(InName))
	{
		return InProcessFunction(**ModifierMetadata);
	}
	return false;
}

UActorModifierCoreSharedObject* UActorModifierCoreSubsystem::GetModifierSharedObject(ULevel* InLevel, TSubclassOf<UActorModifierCoreSharedObject> InClass, bool bInCreateIfNone) const
{
	if (!InClass.Get())
	{
		return nullptr;
	}

	if (AActorModifierCoreSharedActor* SharedActor = GetModifierSharedProvider(InLevel, bInCreateIfNone))
	{
		if (bInCreateIfNone)
		{
			return SharedActor->FindOrAddShared(InClass);
		}
		else
		{
			return SharedActor->FindShared(InClass);
		}
	}

	return nullptr;
}

bool UActorModifierCoreSubsystem::EnableModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, bool bInEnabled, bool bInShouldTransact) const
{
	if (InModifiers.IsEmpty())
	{
		return false;
	}

#if WITH_EDITOR
	// create transaction
	FText TransactionText;
	if (bInEnabled)
	{
		TransactionText = FText::Format(
			LOCTEXT("EnableModifiers.Enable", "Enabling {0} modifier(s)"),
			FText::FromString(FString::FromInt(InModifiers.Num())));
	}
	else
	{
		TransactionText = FText::Format(
			LOCTEXT("EnableModifiers.Disable", "Disabling {0} modifier(s)"),
			FText::FromString(FString::FromInt(InModifiers.Num())));
	}
	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	TSet<UActorModifierCoreBase*> ModifiersSet = InModifiers;

	// lets group modifiers to batch the operation, better to update the stack only once instead of many times
	while (!ModifiersSet.IsEmpty())
	{
		TSet<UActorModifierCoreBase*> StackModifiers;

		const UActorModifierCoreStack* CurrentStack = nullptr;

		for (UActorModifierCoreBase* Modifier : ModifiersSet)
		{
			if (!IsValid(Modifier))
			{
				continue;
			}

			UActorModifierCoreStack* ModifierStack = Modifier->GetModifierStack();
			ModifierStack = IsValid(ModifierStack) ? ModifierStack : Modifier->GetRootModifierStack();

			if (!CurrentStack)
			{
				CurrentStack = ModifierStack;
			}

			if (ModifierStack == CurrentStack)
			{
				StackModifiers.Add(Modifier);
			}
		}

		{
			// disable stack modifiers update
			FActorModifierCoreScopedLock Lock(StackModifiers);
			for (UActorModifierCoreBase* Modifier : StackModifiers)
			{
				Modifier->SetModifierEnabled(bInEnabled);
			}
		}

		ModifiersSet = ModifiersSet.Difference(StackModifiers);
	}

	return true;
}

bool UActorModifierCoreSubsystem::RemoveModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, FActorModifierCoreStackRemoveOp& InRemoveOp) const
{
	if (InModifiers.IsEmpty())
	{
		return false;
	}

#if WITH_EDITOR
	// create transaction
	FText TransactionText;
	const UActorModifierCoreBase* SingleModifier = InModifiers.Array()[0];
	if (InModifiers.Num() == 1 && IsValid(SingleModifier))
	{
		TransactionText = FText::Format(
			LOCTEXT("RemoveSingleModifiers", "Removing {0} modifier"),
			FText::FromName(SingleModifier->GetModifierName()));
	}
	else
	{
		TransactionText = FText::Format(
			LOCTEXT("RemoveMultipleModifiers", "Removing {0} modifier(s)"),
			FText::FromString(FString::FromInt(InModifiers.Num())));
	}
	FScopedTransaction Transaction(TransactionText, InRemoveOp.bShouldTransact);
#endif

	TSet<UActorModifierCoreBase*> ModifiersSet = InModifiers;

	// lets group modifiers to batch the operation, better to update the stack only once instead of many times
	while (!ModifiersSet.IsEmpty())
	{
		TSet<UActorModifierCoreBase*> StackModifiers;

		UActorModifierCoreStack* CurrentStack = nullptr;

		for (UActorModifierCoreBase* Modifier : ModifiersSet)
		{
			if (!IsValid(Modifier))
			{
				continue;
			}

			UActorModifierCoreStack* ModifierStack = Modifier->GetModifierStack();
			ModifierStack = IsValid(ModifierStack) ? ModifierStack : Modifier->GetRootModifierStack();

			if (!CurrentStack)
			{
				CurrentStack = ModifierStack;
			}

			if (ModifierStack == CurrentStack)
			{
				StackModifiers.Add(Modifier);
			}
		}

		// Sort by their order in stack first to avoid dependencies errors
		bool bSuccess = true;
		StackModifiers.Sort([CurrentStack](UActorModifierCoreBase& A, UActorModifierCoreBase& B)
		{
			bool bSortResult = true;
			CurrentStack->ProcessFunction([&A, &B, &bSortResult](const UActorModifierCoreBase* InModifier)->bool
			{
				if (InModifier == &A || InModifier == &B)
				{
					bSortResult = InModifier == &A ? false : true;
					return false;
				}
				return true;
			});
			return bSortResult;
		});

		CurrentStack->ProcessLockFunction([this, &StackModifiers, CurrentStack, &bSuccess, &InRemoveOp]()
		{
			for (UActorModifierCoreBase* RemoveModifier : StackModifiers)
			{
				InRemoveOp.RemoveModifier = RemoveModifier;
				if (!CurrentStack->RemoveModifier(InRemoveOp))
				{
					bSuccess = false;
					break;
				}
			}
		});

		if (!bSuccess)
		{
			return false;
		}

		ModifiersSet = ModifiersSet.Difference(StackModifiers);
	}

	return true;
}

bool UActorModifierCoreSubsystem::RemoveModifierStacks(const TSet<UActorModifierCoreStack*>& InStacks, bool bInShouldTransact) const
{
	if (InStacks.IsEmpty())
	{
		return false;
	}

#if WITH_EDITOR
	const FText TransactionText = LOCTEXT("RemoveModifierComponent", "Removing {0} modifier component(s)");
	const FText StackCount = FText::FromString(FString::FromInt(InStacks.Num()));

	FScopedTransaction Transaction(FText::Format(TransactionText, StackCount), bInShouldTransact);
#endif

	for (UActorModifierCoreStack* ActorStack : InStacks)
	{
		if (!IsValid(ActorStack))
		{
			continue;
		}

		UActorModifierCoreComponent* Component = ActorStack->GetTypedOuter<UActorModifierCoreComponent>();

		if (!IsValid(Component))
		{
			continue;
		}

		AActor* Actor = Component->GetOwner();

		if (!IsValid(Actor))
		{
			continue;
		}

#if WITH_EDITOR
		Actor->Modify();
		Component->Modify();
		ActorStack->Modify();
#endif

		Component->DestroyComponent(/** PromoteChildren */false);
	}

	return true;
}

bool UActorModifierCoreSubsystem::RemoveActorsModifiers(const TSet<AActor*>& InActors, bool bInShouldTransact) const
{
	if (InActors.IsEmpty())
	{
		return false;
	}

	// Get actors stack
	TArray<UActorModifierCoreStack*> ActorStacks;
	for (const AActor* Actor : InActors)
	{
		if (!IsValid(Actor))
		{
			continue;
		}

		if (UActorModifierCoreStack* ActorStack = GetActorModifierStack(Actor))
		{
			ActorStacks.Add(ActorStack);
		}
	}

#if WITH_EDITOR
	// create transaction
	const FText TransactionText = FText::Format(
		LOCTEXT("RemoveModifier", "Remove all modifiers from {0} actor(s)"),
		FText::FromString(FString::FromInt(InActors.Num())));
	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	// remove modifiers from actors
	bool bSuccess = true;
	for (UActorModifierCoreStack* ActorStack : ActorStacks)
	{
		if (!ActorStack->RemoveAllModifiers())
		{
			bSuccess = false;
		}
	}

	return bSuccess;
}

TArray<UActorModifierCoreBase*> UActorModifierCoreSubsystem::AddActorsModifiers(const TSet<AActor*>& InActors, FActorModifierCoreStackInsertOp& InAddOp) const
{
	TArray<UActorModifierCoreBase*> NewModifiers;

	if (!IsRegisteredModifierClass(InAddOp.NewModifierName))
	{
		return NewModifiers;
	}

	// Get actors stack, create if none
	TArray<UActorModifierCoreStack*> ActorStacks;
	for (AActor* Actor : InActors)
	{
		if (!IsValid(Actor))
		{
			continue;
		}

		UActorModifierCoreStack* ActorStack = GetActorModifierStack(Actor);

		if (!ActorStack)
		{
			ActorStack = AddActorModifierStack(Actor);
		}

		if (IsValid(ActorStack))
		{
			ActorStacks.Add(ActorStack);
		}
	}

#if WITH_EDITOR
	// create transaction
	const FText TransactionText = FText::Format(
		LOCTEXT("AddModifier", "Add {0} modifier on {1} actor(s)"),
		FText::FromName(InAddOp.NewModifierName),
		FText::FromString(FString::FromInt(InActors.Num())));
	FScopedTransaction Transaction(TransactionText, InAddOp.bShouldTransact);
#endif

	// add modifier to actors
	InAddOp.InsertPositionContext = nullptr;
	InAddOp.InsertPosition = EActorModifierCoreStackPosition::Before;

	OnInsertModifier(InAddOp);

	for (UActorModifierCoreStack* ActorStack : ActorStacks)
	{
		NewModifiers.Add(ActorStack->InsertModifier(InAddOp));
	}

	return NewModifiers;
}

UActorModifierCoreBase* UActorModifierCoreSubsystem::InsertModifier(UActorModifierCoreStack* InStack, FActorModifierCoreStackInsertOp& InInsertOp) const
{
	UActorModifierCoreBase* NewModifier = nullptr;

	if (!IsRegisteredModifierClass(InInsertOp.NewModifierName))
	{
		return NewModifier;
	}

	if (!IsValid(InStack))
	{
		return NewModifier;
	}

#if WITH_EDITOR
	// create transaction
	FText TransactionText;
	if (InInsertOp.InsertPosition == EActorModifierCoreStackPosition::Before)
	{
		if (IsValid(InInsertOp.InsertPositionContext))
		{
			TransactionText = FText::Format(
				LOCTEXT("InsertModifierBefore", "Insert {0} modifier before {1}"),
				FText::FromName(InInsertOp.NewModifierName),
				FText::FromName(InInsertOp.InsertPositionContext->GetModifierName()));
		}
		else
		{
			TransactionText = FText::Format(
				LOCTEXT("InsertModifierAtEnd", "Insert {0} modifier at the end of stack"),
				FText::FromName(InInsertOp.NewModifierName));
		}
	}
	else if (InInsertOp.InsertPosition == EActorModifierCoreStackPosition::After)
	{
		if (IsValid(InInsertOp.InsertPositionContext))
		{
			TransactionText = FText::Format(
				LOCTEXT("InsertModifierAfter", "Insert {0} modifier after {1}"),
				FText::FromName(InInsertOp.NewModifierName),
				FText::FromName(InInsertOp.InsertPositionContext->GetModifierName()));
		}
		else
		{
			TransactionText = FText::Format(
				LOCTEXT("InsertModifierAtStart", "Insert {0} modifier at the start of stack"),
				FText::FromName(InInsertOp.NewModifierName));
		}
	}
	FScopedTransaction Transaction(TransactionText, InInsertOp.bShouldTransact);
#endif

	OnInsertModifier(InInsertOp);

	// insert modifier in stack
	NewModifier = InStack->InsertModifier(InInsertOp);

	return NewModifier;
}

bool UActorModifierCoreSubsystem::MoveModifier(UActorModifierCoreStack* InStack, FActorModifierCoreStackMoveOp& InMoveOp) const
{
	if (!IsValid(InMoveOp.MoveModifier))
	{
		return false;
	}

	UActorModifierCoreStack* ModifierStack = InMoveOp.MoveModifier->GetModifierStack();

	if (!IsValid(ModifierStack) || ModifierStack != InStack)
	{
		return false;
	}

	const FName& MoveModifierName = InMoveOp.MoveModifier->GetModifierName();

#if WITH_EDITOR
	// create transaction
	FText TransactionText;
	if (InMoveOp.MovePosition == EActorModifierCoreStackPosition::Before)
	{
		if (IsValid(InMoveOp.MovePositionContext))
		{
			TransactionText = FText::Format(
				LOCTEXT("MoveModifierBefore", "Move {0} modifier before {1}"),
				FText::FromName(MoveModifierName),
				FText::FromName(InMoveOp.MovePositionContext->GetModifierName()));
		}
		else
		{
			TransactionText = FText::Format(
				LOCTEXT("MoveModifierToEnd", "Move {0} modifier at the end of stack"),
				FText::FromName(MoveModifierName));
		}
	}
	else if (InMoveOp.MovePosition == EActorModifierCoreStackPosition::After)
	{
		if (IsValid(InMoveOp.MovePositionContext))
		{
			TransactionText = FText::Format(
				LOCTEXT("MoveModifierAfter", "Move {0} modifier after {1}"),
				FText::FromName(MoveModifierName),
				FText::FromName(InMoveOp.MovePositionContext->GetModifierName()));
		}
		else
		{
			TransactionText = FText::Format(
				LOCTEXT("MoveModifierToStart", "Move {0} modifier at the start of stack"),
				FText::FromName(MoveModifierName));
		}
	}
	FScopedTransaction Transaction(TransactionText, InMoveOp.bShouldTransact);
#endif

	// move modifier in stack
	return ModifierStack->MoveModifier(InMoveOp);
}

AActorModifierCoreSharedActor* UActorModifierCoreSubsystem::GetModifierSharedProvider(ULevel* InLevel, bool bInSpawnIfNotFound) const
{
	if (!InLevel)
	{
		return nullptr;
	}

	UWorld* World = InLevel->GetWorld();

	if (!World)
	{
		return nullptr;
	}

	// return cache one if already registered
	const TWeakObjectPtr<AActorModifierCoreSharedActor>* SharedProvider = ModifierSharedProviders.Find(InLevel);
	if (SharedProvider && SharedProvider->IsValid())
	{
		return SharedProvider->Get();
	}

	// Take only first one into account and register it
	for(AActorModifierCoreSharedActor* Actor : TActorRange<AActorModifierCoreSharedActor>(World))
	{
		if (Actor->GetLevel() == InLevel)
		{
			RegisterModifierSharedProvider(Actor);
			return Actor;
		}
	}

	if (!bInSpawnIfNotFound)
	{
		return nullptr;
	}

	// Spawn new one and register it
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = InLevel;
#if WITH_EDITOR
	SpawnParameters.bHideFromSceneOutliner = true;
#endif

	AActorModifierCoreSharedActor* const NewSharedActor = World->SpawnActor<AActorModifierCoreSharedActor>(SpawnParameters);
	RegisterModifierSharedProvider(NewSharedActor);

	return NewSharedActor;
}

bool UActorModifierCoreSubsystem::RegisterModifierSharedProvider(AActorModifierCoreSharedActor* InSharedActor) const
{
	if (!InSharedActor)
	{
		return false;
	}

	ULevel* Level = InSharedActor->GetLevel();
	if (!Level)
	{
		return false;
	}

	if (ModifierSharedProviders.Contains(Level))
	{
		return false;
	}

	UActorModifierCoreSubsystem* MutableThis = const_cast<UActorModifierCoreSubsystem*>(this);
	MutableThis->ModifierSharedProviders.Add(Level, InSharedActor);

	UE_LOGF(LogActorModifierCoreSubsystem, Verbose, "Modifier shared provider registered for world %ls and level %ls", *Level->GetWorld()->GetDebugDisplayName(), *Level->GetName());

	return true;
}

bool UActorModifierCoreSubsystem::UnregisterModifierSharedProvider(const AActor* InSharedActor) const
{
	if (!InSharedActor)
	{
		return false;
	}

	const ULevel* Level = InSharedActor->GetLevel();
	if (!Level)
	{
		return false;
	}

	const TWeakObjectPtr<AActorModifierCoreSharedActor>* SharedActor = ModifierSharedProviders.Find(Level);
	if (!SharedActor)
	{
		return false;
	}

	if (SharedActor->Get() != InSharedActor)
	{
		return false;
	}

	UE_LOGF(LogActorModifierCoreSubsystem, Verbose, "Modifier shared provider unregistered for world %ls and level %ls", *Level->GetWorld()->GetDebugDisplayName(), *Level->GetName());

	UActorModifierCoreSubsystem* MutableThis = const_cast<UActorModifierCoreSubsystem*>(this);
	return MutableThis->ModifierSharedProviders.Remove(Level) > 0;
}

void UActorModifierCoreSubsystem::ScanForModifiers()
{
	for (const UClass* const Class : TObjectRange<UClass>())
	{
		RegisterModifierClass(Class);
	}
}

void UActorModifierCoreSubsystem::OnAssetRegistryFilesLoaded()
{
	bFilesLoaded = true;

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> Assets;
	const FTopLevelAssetPath BlueprintPath(BlueprintClass);
	AssetRegistryModule.Get().GetAssetsByClass(BlueprintPath, Assets, /** Subclass */true);

	for (const FAssetData& Asset : Assets)
	{
		RegisterModifierAsset(Asset);
	}
}

void UActorModifierCoreSubsystem::OnAssetRegistryAssetAdded(const FAssetData& InAssetData)
{
	if (bFilesLoaded)
	{
		RegisterModifierAsset(InAssetData);
	}
}

void UActorModifierCoreSubsystem::OnAssetRegistryAssetUpdated(const FAssetData& InAssetData)
{
	RegisterModifierAsset(InAssetData);
}

void UActorModifierCoreSubsystem::OnAssetRegistryAssetRemoved(const FAssetData& InAssetData)
{
	UnregisterModifierAsset(InAssetData);
}

void UActorModifierCoreSubsystem::OnBlueprintObjectsReplaced(const TMap<UObject*, UObject*>& InReplacements)
{
	for (const TPair<UObject*, UObject*>& Replacement : InReplacements)
	{
		// Replace modifiers that are not the template
		if (!Replacement.Key || !Replacement.Value || Replacement.Key->IsTemplate() || Replacement.Value->IsTemplate())
		{
			continue;
		}

		UActorModifierCoreBlueprintBase* ReplacedModifier = Cast<UActorModifierCoreBlueprintBase>(Replacement.Key);
		UActorModifierCoreBlueprintBase* ReplacementModifier = Cast<UActorModifierCoreBlueprintBase>(Replacement.Value);

		if (ReplacedModifier && ReplacementModifier)
		{
			UnregisterModifierClass(ReplacedModifier->GetClass());
			UnregisterModifierClass(ReplacementModifier->GetClass());

			RegisterModifierClass(ReplacementModifier->GetClass());

			OnModifierReplacedDelegate.Broadcast(ReplacedModifier, ReplacementModifier);
		}
	}
}

void UActorModifierCoreSubsystem::RegisterModifierAsset(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath.ToString().Equals(BlueprintClass))
	{
		FString GeneratedClassPath;
		if (InAssetData.GetTagValue(TEXT("GeneratedClass"), GeneratedClassPath))
		{
			if (const UClass* GeneratedClass = LoadObject<UClass>(nullptr, *GeneratedClassPath))
			{
				UnregisterModifierClass(GeneratedClass);

				if (RegisterModifierClass(GeneratedClass))
				{
#if WITH_EDITOR
					if (FEngineAnalytics::IsAvailable())
					{
						TArray<FAnalyticsEventAttribute> Attributes;
						Attributes.Emplace(TEXT("Class"), GeneratedClassPath);
						FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ActorModifiers.RegisterBlueprintModifier"), Attributes);
					}
#endif
				}
			}
		}
	}
}

void UActorModifierCoreSubsystem::UnregisterModifierAsset(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath.ToString().Equals(BlueprintClass))
	{
		FString GeneratedClassPath;
		if (InAssetData.GetTagValue(TEXT("GeneratedClass"), GeneratedClassPath))
		{
			if (const UClass* GeneratedClass = LoadObject<UClass>(nullptr, *GeneratedClassPath))
			{
				UnregisterModifierClass(GeneratedClass);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
