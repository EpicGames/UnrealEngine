// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Components/ActorComponent.h"
#include "CoreTypes.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierCoreDefs.h"
#include "Modifiers/ActorModifierCoreExtension.h"
#include "Profiler/ActorModifierCoreProfiler.h"
#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "Misc/GeneratedTypeName.h"
#include "UObject/Object.h"

#include "ActorModifierCoreBase.generated.h"

class UActorModifierCoreComponent;
class UActorModifierCoreSharedObject;
class UActorModifierCoreStack;

/** Abstract base class for all modifier, a modifier must be located in a modifier stack to work properly */
UCLASS(MinimalAPI, NotBlueprintable, Abstract, EditInlineNew, HideCategories=(Tags, AssetUserData, Activation, Collision, Cooking))
class UActorModifierCoreBase : public UObject
{
	friend class UActorModifierCoreComponent;
	friend class UActorModifierCoreStack;
	friend class UActorModifierCoreSubsystem;

	friend class FActorModifierCoreEditorDetailCustomization;
	friend class UActorModifierCoreEditorStackCustomization;
	
	friend struct FActorModifierCoreExecutionTask;
	friend struct FActorModifierCoreMetadata;
	friend struct FActorModifierCoreScopedLock;

	GENERATED_BODY()

public:
	/** Returns modifier metadata */
	const FActorModifierCoreMetadata& GetModifierMetadata() const;

	/** Returns this modifier unique name */
	ACTORMODIFIERCORE_API FName GetModifierName() const;

	/** Returns this modifier category */
	ACTORMODIFIERCORE_API FName GetModifierCategory() const;

	/** Gets the modifier profiler */
	TSharedPtr<FActorModifierCoreProfiler> GetProfiler() const
	{
		return Profiler;
	}

	/** Gets the modifier last status */
	const FActorModifierCoreStatus& GetModifierLastStatus() const
	{
		return Status;
	}

	/** Whether this modifier is a stack to avoid casting */
	ACTORMODIFIERCORE_API bool IsModifierStack() const;

	/** Returns the modified actor for this modifier */
	ACTORMODIFIERCORE_API AActor* GetModifiedActor() const;

	/** Returns the stack this modifier is in */
	ACTORMODIFIERCORE_API UActorModifierCoreStack* GetModifierStack() const;

	/** Returns the top root stack this modifier is in */
	ACTORMODIFIERCORE_API UActorModifierCoreStack* GetRootModifierStack() const;

	/** Returns the modifier this stack is in */
	UActorModifierCoreComponent* GetModifierComponent() const;

	/** Returns the previous modifier found before this one, will recurse upwards, could return a stack */
	const UActorModifierCoreBase* GetPreviousModifier() const;

	/** Returns the next modifier found after this one, will recurse upwards, could return a stack */
	const UActorModifierCoreBase* GetNextModifier() const;

	/** Returns the first modifier before this one that matches the name, will recurse upwards, could return a stack */
	const UActorModifierCoreBase* GetPreviousNameModifier(const FName& InModifierName) const;

	/** Returns the first modifier after this one that matches the name, will recurse upwards, could return a stack */
	const UActorModifierCoreBase* GetNextNameModifier(const FName& InModifierName) const;

	/** Returns whether a modifier can be marked dirty */
	ACTORMODIFIERCORE_API bool CanMarkModifierDirty() const;

	/** 
	 * Set this modifier as dirty to be able to execute only dirty modifiers 
	 * returns true if the modifier could be marked dirty
	 */
	ACTORMODIFIERCORE_API bool MarkModifierDirty(bool bExecute = true);

	/** Whether this modifier is set as dirty */
	bool IsModifierDirty() const
	{
		return bModifierDirty;
	}

	/** Whether this modifier is idle and not running */
	bool IsModifierIdle() const
	{
		return bModifierIdle;
	}

	/** Is the execution of this modifier locked */
	bool IsModifierExecutionLocked() const
	{
		return bModifierExecutionLocked;
	}

	/** Whether this modifier is applied and can be restored */
	bool IsModifierApplied() const
	{
		return bModifierApplied;
	}

	/** Whether this modifier was initialized and is now ready to operate */
	bool IsModifierInitialized() const
	{
		return bModifierInitialized;
	}

	/** Whether this modifier contains other modifiers */
	virtual bool IsModifierEmpty() const
	{
		return false;
	}

	UFUNCTION()
	void SetModifierEnabled(bool bInEnabled);

	ACTORMODIFIERCORE_API bool IsModifierEnabled() const;

	/** Locks this modifier execution and process a function before unlocking it */
	ACTORMODIFIERCORE_API void ProcessLockFunction(TFunctionRef<void()> InFunction);

	/** Is this modifier in profiling mode */
	ACTORMODIFIERCORE_API bool IsModifierProfiling() const;

	/** Helper to lookup components type, by default will loop through this actor components, stops when false is returned */
	template<typename InComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<InComponentClass, UActorComponent>::Value>::Type>
	bool ForEachComponent(TFunctionRef<bool(InComponentClass*)> InFunction, EActorModifierCoreComponentType InComponentType = EActorModifierCoreComponentType::Owned, EActorModifierCoreLookup InActorLookup = EActorModifierCoreLookup::Self, TOptional<const AActor*> InOptionalActor = TOptional<const AActor*>()) const
	{
		if (!InOptionalActor.IsSet())
		{
			InOptionalActor = ModifiedActor.Get();
		}

		if (!InOptionalActor.GetValue())
		{
			return false;
		}

		return ForEachActor<AActor>([InComponentType, &InFunction](const AActor* InActor)->bool
		{
			if (EnumHasAnyFlags(InComponentType, EActorModifierCoreComponentType::Owned))
			{
				for (UActorComponent* ActorComponent : InActor->GetComponents())
				{
					if (ActorComponent && ActorComponent->IsA<InComponentClass>())
					{
						if (!InFunction(Cast<InComponentClass>(ActorComponent)))
						{
							return false;
						}
					}
				}
			}
			if (EnumHasAnyFlags(InComponentType, EActorModifierCoreComponentType::Instanced))
			{
				for (UActorComponent* ActorComponent : InActor->GetInstanceComponents())
				{
					if (ActorComponent && ActorComponent->IsA<InComponentClass>())
					{
						if (!InFunction(Cast<InComponentClass>(ActorComponent)))
						{
							return false;
						}
					}
				}
			}
			return true;
		}, InActorLookup, InOptionalActor);
	}

	/** Helper to lookup actors, by default will loop through this actor direct children, stops when false is returned */
	template<typename InActorClass = AActor, typename = typename TEnableIf<TIsDerivedFrom<InActorClass, AActor>::Value>::Type>
	bool ForEachActor(TFunctionRef<bool(InActorClass*)> InFunction, EActorModifierCoreLookup InActorLookup = EActorModifierCoreLookup::DirectChildren, TOptional<const AActor*> InOptionalActor = TOptional<const AActor*>()) const
	{
		if (!InOptionalActor.IsSet())
		{
			InOptionalActor = ModifiedActor.Get();
		}

		if (!InOptionalActor.GetValue())
		{
			return false;
		}

		TArray<AActor*> CheckActors;
		if (EnumHasAnyFlags(InActorLookup, EActorModifierCoreLookup::Self))
		{
			CheckActors.Add(const_cast<AActor*>(InOptionalActor.GetValue()));
		}
		if (EnumHasAnyFlags(InActorLookup, EActorModifierCoreLookup::DirectChildren))
		{
			const bool bIncludeNested = EnumHasAnyFlags(InActorLookup, EActorModifierCoreLookup::AllChildren);
			InOptionalActor.GetValue()->GetAttachedActors(CheckActors, false, bIncludeNested);
		}

		for (AActor* Actor : CheckActors)
		{
			if (Actor)
			{
				if (!InFunction(Actor))
				{
					return false;
				}
			}
		}

		return true;
	}

protected:
	ACTORMODIFIERCORE_API UActorModifierCoreBase();

	//~ Begin UObject
	ACTORMODIFIERCORE_API virtual void PostLoad() override;
	ACTORMODIFIERCORE_API virtual void PostEditImport() override;
	ACTORMODIFIERCORE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#if WITH_EDITOR
	ACTORMODIFIERCORE_API virtual void PreEditUndo() override;
	ACTORMODIFIERCORE_API virtual void PostEditUndo() override;
	ACTORMODIFIERCORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Override metadata for this modifier, called only once before modifier CDO is registered */
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) {}

	/** Checks whether this modifier is ready to run, called before this modifier is executed */
	virtual bool IsModifierReady() const { return true; }

	/** You can do additional lightweights checks here in case you want to dirty the modifier instead of waiting for an event, tick must be enabled in the metadata */
	virtual bool IsModifierDirtyable() const { return false; }

	/** Override in child classes, called before applying this modifier */
	virtual void SavePreState() {}

	/** Override in child classes, unapply this modified changes and restore the state before this modifier is applied */
	virtual void RestorePreState() {}

	/** Override in child classes, apply this modifier on the actual actor, call Next or Fail to complete the modifier execution */
	ACTORMODIFIERCORE_API virtual void Apply();

	/** Called after the modifier was added to a stack and initialized after creation or serialization or duplication */
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) {}

	/** Called after the modifier is removed from a stack and uninitialized after actor destruction */
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) {}

	/** Called when a modifier is enabled */
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) {}

	/** Called when a modifier is disabled */
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) {}

	/** Called when modified actor transform is updated if this modifier is enabled */
	virtual void OnModifiedActorTransformed() {}

	/** The modifier was executed properly, skip to the next one */
	ACTORMODIFIERCORE_API void Next();

	/** The modifier failed executing, do not continue and stop here */
	ACTORMODIFIERCORE_API void Fail(const FText& InFailReason);

	/** Get shared object of that specific class or create one if none was found, casted version */
	template<typename InSharedClass = UActorModifierCoreSharedObject, typename = typename TEnableIf<TIsDerivedFrom<InSharedClass, UActorModifierCoreSharedObject>::Value>::Type>
	InSharedClass* GetShared(bool bInCreateIfNone = false) const
	{
		UClass* SharedClass = InSharedClass::StaticClass();
		return Cast<InSharedClass>(GetShared(SharedClass, bInCreateIfNone));
	}

	/** Get shared object of that specific class or create one if none was found */
	ACTORMODIFIERCORE_API UActorModifierCoreSharedObject* GetShared(TSubclassOf<UActorModifierCoreSharedObject> InClass, bool bInCreateIfNone = false) const;

	/** Adds an extension of a specific type to this modifier with its arguments or gets it if it already exists */
	template<typename InExtensionClass, typename = typename TEnableIf<TIsDerivedFrom<InExtensionClass, FActorModifierCoreExtension>::IsDerived>::Type, typename... InArgTypes>
	InExtensionClass* AddExtension(InArgTypes&& ... InArgs)
	{
		const FName ExtensionType = GetGeneratedTypeName<InExtensionClass>();
		if (ExtensionType.IsNone())
		{
			return nullptr;
		}

		if (InExtensionClass* Extension = GetExtension<InExtensionClass>())
		{
			return Extension;
		}

		TSharedPtr<InExtensionClass> NewExtension = MakeShared<InExtensionClass>(Forward<InArgTypes>(InArgs)...);

		AddExtensionInternal(ExtensionType, NewExtension);

		return NewExtension.Get();
	}

	/** Executes the function for that extension if it exists, returns true if execution was successful */
	template<typename InExtensionClass, typename = typename TEnableIf<TIsDerivedFrom<InExtensionClass, FActorModifierCoreExtension>::IsDerived>::Type>
	bool ProcessExtension(TFunctionRef<void(InExtensionClass*)> InFunction)
	{
		const FName ExtensionType = GetGeneratedTypeName<InExtensionClass>();
		if (ExtensionType.IsNone())
		{
			return false;
		}

		if (InExtensionClass* Extension = GetExtension<InExtensionClass>(ExtensionType))
		{
			InFunction(Extension);
			return true;
		}

		return false;
	}

	/** Gets an extension with this type, casted version */
	template<typename InExtensionClass, typename = typename TEnableIf<TIsDerivedFrom<InExtensionClass, FActorModifierCoreExtension>::IsDerived>::Type>
	InExtensionClass* GetExtension() const
	{
		const FName ExtensionType = GetGeneratedTypeName<InExtensionClass>();
		if (ExtensionType.IsNone())
		{
			return nullptr;
		}

		if (FActorModifierCoreExtension* Extension = GetExtension(ExtensionType))
		{
			return static_cast<InExtensionClass*>(Extension);
		}

		return nullptr;
	}

	/** Gets an extension of this specific type */
	ACTORMODIFIERCORE_API FActorModifierCoreExtension* GetExtension(const FName& InExtensionType) const;

	/** Removes an extension of this type from this modifier */
	ACTORMODIFIERCORE_API bool RemoveExtension(const FName& InExtensionType);

	UE_DEPRECATED(5.8, "LogModifier deprecated. Prefer using UE_LOG instead.")
	ACTORMODIFIERCORE_API void LogModifier(const FString& InLog, bool bInForce = false, EActorModifierCoreStatus InType = EActorModifierCoreStatus::Success) const;

	void DeferInitializeModifier();

private:
	/** Called when modifier becomes dirty */
	ACTORMODIFIERCORE_API virtual void OnModifierDirty(UActorModifierCoreBase* DirtyModifier, bool bExecute);

	/** Execute a const function on this modifier, only to read data */
	ACTORMODIFIERCORE_API virtual bool ProcessFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions) const;

	/** INTERNAL USE ONLY, called by the stack only to unapply this modifier if it was applied */
	void Unapply();

	/** INTERNAL USE ONLY, called after load or creation or duplication */
	void InitializeModifier(EActorModifierCoreEnableReason InReason);

	/** INTERNAL USE ONLY, called before being destroyed or removed */
	void UninitializeModifier(EActorModifierCoreDisableReason InReason);

	/** INTERNAL USE ONLY, called once after creating a modifier to set inner properties */
	void PostModifierCreation(UActorModifierCoreStack* InStack);

	/** INTERNAL USE ONLY, called once after creating the CDO modifier */
	void PostModifierCDOCreation();

	/** INTERNAL USE ONLY, Called when the world and all its actors have loaded to execute initialized modifiers */
	void PostModifierWorldLoad(UWorld* InWorld, ELevelTick InType, float InDelta);

	/** INTERNAL USE ONLY, handles enable virtual calls and other required tasks */
	void EnableModifier(EActorModifierCoreEnableReason InReason);

	/** INTERNAL USE ONLY, handles disable virtual calls and other required tasks */
	void DisableModifier(EActorModifierCoreDisableReason InReason);

	/** INTERNAL USE ONLY, locks the modifier execution to prevent the execution of the stack it is in */
	void LockModifierExecution();

	/** INTERNAL USE ONLY, unlocks the modifier and executes it if marked dirty */
	void UnlockModifierExecution();

	/** INTERNAL USE ONLY, add and setup a newly created extension for this modifier */
	ACTORMODIFIERCORE_API void AddExtensionInternal(const FName& InExtensionType, TSharedPtr<FActorModifierCoreExtension> InExtension);

	/** Called after setting the modifier enabled state */
	void OnModifierEnabledChanged(bool bInExecute = true);

	/** Called before the execution of this modifier */
	void BeginModifierExecution();

	/** Called after the execution of this modifier */
	void EndModifierExecution();

	// TODO Update Strategy : UpdateSelf, UpdateAll, UpdateAfter, UpdateSameCategories

	/** Map of typed extensions, will follow this modifier lifecycle */
	TMap<FName, TSharedPtr<FActorModifierCoreExtension, ESPMode::ThreadSafe>> ModifierExtensions;

	TWeakObjectPtr<AActor> ModifiedActor = nullptr;

	/** Is the modifier enabled or disabled */
	UPROPERTY(EditInstanceOnly, Setter="SetModifierEnabled", Getter="IsModifierEnabled", Category="Modifier", meta=(DisplayName="Enable Modifier"))
	bool bModifierEnabled = true;

	/** Is the modifier idle and not running */
	bool bModifierIdle = true;

	/** Is the modifier dirty and needs an update */
	bool bModifierDirty = false;

	/** Is the modifier applied and whether it can be un-applied */
	bool bModifierApplied = false;

	/** Only initialize a modifier once, after creation or load */
	bool bModifierInitialized = false;

	/** When marked dirty will not execute and prevent stack from executing */
	bool bModifierExecutionLocked = false;

	/** This modifier is optimized in the stack */
	bool bModifierOptimized = false;

	/** This modifier metadata, same metadata for this modifier class */
	TSharedPtr<FActorModifierCoreMetadata> Metadata;

	/** This modifier profiler */
	TSharedPtr<FActorModifierCoreProfiler> Profiler;

	/** This modifier last status message */
	FActorModifierCoreStatus Status;
};
