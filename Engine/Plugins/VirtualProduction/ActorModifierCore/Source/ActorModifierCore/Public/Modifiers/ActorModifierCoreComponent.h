// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ActorModifierCoreComponent.generated.h"

class UActorModifierCoreStack;

/** Component that contains a modifier stack and can be added on any actor */
UCLASS(MinimalAPI, ClassGroup="Modifiers", BlueprintType, EditInlineNew, DefaultToInstanced, meta=(BlueprintSpawnableComponent), HideCategories=(Tags, Activation, Cooking, AssetUserData, Collision))
class UActorModifierCoreComponent : public UActorComponent
{
	GENERATED_BODY()

	friend class UActorModifierCoreBase;

public:
	/** Create this component for a specific actor and exposes it to the editor and for serialization */
	static UActorModifierCoreComponent* CreateAndExposeComponent(AActor* InParentActor);

	UActorModifierCoreComponent();

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers")
	UActorModifierCoreStack* GetModifierStack() const
	{
		return ModifierStack;
	}

protected:
	//~ Begin UActorComponent
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ End UActorComponent

	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	virtual void TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction) override;

	/** This is the root stack that will contain all modifiers for this actor */
	UPROPERTY(VisibleInstanceOnly, NoClear, Export, Instanced, Category="Modifiers")
	TObjectPtr<UActorModifierCoreStack> ModifierStack = nullptr;
};