// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Misc/GeneratedTypeName.h"
#include "Profiler/ActorModifierCoreProfiler.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#if WITH_EDITOR
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#endif

#include "ActorModifierCoreDefs.generated.h"

class FActorModifierCoreProfiler;
class AActor;
class FText;
class UActorModifierCoreBase;
class UActorModifierCoreStack;

UENUM(BlueprintType)
enum class EActorModifierCoreEnableReason : uint8
{
	/** Modifier added by user */
	User,
	/** Modifier added by load */
	Load,
	/** Modifier added by undo */
	Undo,
	/** Modifier added by duplicate */
	Duplicate
};

UENUM(BlueprintType)
enum class EActorModifierCoreDisableReason : uint8
{
	/** Modifier disabled by user */
	User,
	/** Modifier disabled by undo */
	Undo,
	/** Modifier disabled by actor destroyed */
	Destroyed
};

/** Enumerates valid positions for modifier operations */
UENUM(BlueprintType)
enum class EActorModifierCoreStackPosition : uint8
{
	Before,
	After
};

/** Enumerates component type on actor */
enum class EActorModifierCoreComponentType : uint8
{
	Owned = 1 << 0,
	Instanced = 1 << 1,
	All = Owned | Instanced
};

/** Enumerates actor lookup possibilities */
enum class EActorModifierCoreLookup : uint8
{
	Self = 1 << 0,
	DirectChildren = 1 << 1,
	SelfAndDirectChild = Self | DirectChildren,
	AllChildren = DirectChildren | 1 << 2,
	SelfAndAllChildren = Self | AllChildren
};

/** Enumerates modifier status message */
enum class EActorModifierCoreStatus : uint8
{
	/** Everything went well */
	Success,
	/** Everything went well but requires user attention */
	Warning,
	/** Something went wrong */
	Error
};

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FModifierCompatibilityRule, const AActor*, InTargetActor);

/** Metadata for each modifier CDO, modifier instance will share same metadata as CDO */
USTRUCT(BlueprintType)
struct FActorModifierCoreMetadata
{
	GENERATED_BODY()

	static inline const FName DefaultCategory = TEXT("Default");
#if WITH_EDITOR
	static inline const FText DefaultDescription = FText::FromString(TEXT("Description not provided"));
	static inline const FLinearColor DefaultColor = FLinearColor::White;
#endif

	FActorModifierCoreMetadata();
	explicit FActorModifierCoreMetadata(const UActorModifierCoreBase* InModifier);

	/** Set the profiler class used for this modifier instances */
	template<typename InProfilerClass, typename = typename TEnableIf<TIsDerivedFrom<InProfilerClass, FActorModifierCoreProfiler>::IsDerived>::Type>
	FActorModifierCoreMetadata& SetProfilerClass()
	{
		ProfilerFunction = [this](UActorModifierCoreBase* InModifier)->TSharedPtr<FActorModifierCoreProfiler>
		{
			TSharedPtr<FActorModifierCoreProfiler> Profiler = MakeShared<InProfilerClass>();
			SetupProfilerInstanceInternal(Profiler, InModifier, GetGeneratedTypeName<InProfilerClass>());
			return Profiler;
		};

		return *this;
	}

	/** Set name of this modifier, should only be set once */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetName(FName InName);

	/** Set Category group of this modifier, should only be set once */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetCategory(FName InCategory);

	/** Allow modifier to tick to update when IsModifierDirtyable returns true */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AllowTick(bool bInAllowed);

	/** Allows multiple modifiers of the same type in the same stack */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AllowMultiple(bool bInAllowed);

	/** Add a modifier dependency for this modifier, will be added when this modifier is added */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AddDependency(const FName& InModifierName);

	/** Disallows this modifier before another modifier */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& DisallowBefore(const FName& InModifierName);

	/** Disallows this modifier after another modifier */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& DisallowAfter(const FName& InModifierName);

	/** Avoid usage of this modifier before another modifier category */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AvoidBeforeCategory(const FName& InCategory);

	/** Avoid usage of this modifier after another modifier category */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AvoidAfterCategory(const FName& InCategory);

	/** Sets the usage rule for this modifier, if it passes it will be available for this actor */
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetCompatibilityRule(const TFunction<bool(const AActor*)>& InModifierRule);

	/** Sets the usage rule for this modifier, if it passes it will be available for this actor */
	FActorModifierCoreMetadata& SetCompatibilityRule(const FModifierCompatibilityRule& InModifierRule);

	/** Create the modifier instance */
	UActorModifierCoreBase* CreateModifierInstance(UActorModifierCoreStack* InStack) const;

	/** Create the profiler instance */
	TSharedPtr<FActorModifierCoreProfiler> CreateProfilerInstance(UActorModifierCoreBase* InModifier) const;

	bool ResetDefault();

	FName GetName() const
	{
		return Name;
	}

	FName GetCategory() const
	{
		return Category;
	}

	bool IsTickAllowed() const
	{
		return bTickAllowed;
	}

	bool IsMultipleAllowed() const
	{
		return bMultipleAllowed;
	}

	bool IsStack() const
	{
		return bIsStack;
	}

	TConstArrayView<FName> GetDependencies() const
	{
		return Dependencies;
	}

	const TSet<FName>& GetDisallowedBefore() const
	{
		return DisallowedBefore;
	}

	const TSet<FName>& GetDisallowedAfter() const
	{
		return DisallowedAfter;
	}

	TSubclassOf<UActorModifierCoreBase> GetClass() const
	{
		return Class;
	}

	bool IsDisallowedAfter(const FName& InModifierName) const;

	bool IsDisallowedBefore(const FName& InModifierName) const;

	bool IsAllowedAfter(const FName& InModifierName) const;

	bool IsAllowedBefore(const FName& InModifierName) const;

	bool IsCompatibleWith(const AActor* InActor) const;

	/** Checks if this modifier depends on another modifier */
	bool DependsOn(const FName& InModifierName) const;

	/** Check if this modifier is required by the other modifier */
	bool IsRequiredBy(const FName& InModifierName) const;

	bool ShouldAvoidBefore(FName InCategory) const;

	bool ShouldAvoidAfter(FName InCategory) const;

#if WITH_EDITOR
	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetDisplayName(const FText& InName);

	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetDescription(const FText& InDescription);

	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetColor(const FLinearColor& InColor);

	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetIcon(const FSlateIcon& InIcon);

	ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetHidden(bool bInHidden);

	bool IsHidden() const
	{
		return bHidden;
	}

	const FText& GetDisplayName() const
	{
		return DisplayName;
	}

	const FText& GetDescription() const
	{
		return Description;
	}

	const FLinearColor& GetColor() const
	{
		return Color;
	}

	const FSlateIcon& GetIcon() const
	{
		return Icon;
	}
#endif

private:
	ACTORMODIFIERCORE_API void SetupProfilerInstanceInternal(TSharedPtr<FActorModifierCoreProfiler> InProfiler, UActorModifierCoreBase* InModifier, const FName& InProfilerType) const;

	/** Unique name to use for this modifier */
	FName Name;

	/** Category group of this modifier to sort them */
	FName Category;

	/** Tick allowed for this modifier to run additional checks to detect changes */
	bool bTickAllowed = false;

	/** Is this modifier allowed multiple times in the same stack */
	bool bMultipleAllowed = false;

	/** Is this modifier a stack */
	bool bIsStack;

	/** What modifiers does this modifier need to work properly, in the correct order */
	TArray<FName> Dependencies;

	/** Is this modifier disallowed before these modifiers */
	TSet<FName> DisallowedBefore;

	/** Is this modifier disallowed after these modifiers */
	TSet<FName> DisallowedAfter;

	/** Modifier avoided before these categories */
	TSet<FName> AvoidedBeforeCategories;

	/** Modifier avoided after these categories */
	TSet<FName> AvoidedAfterCategories;

	/** Rule to pass before this modifier can be used on an actor */
	TFunction<bool(const AActor*)> CompatibilityRuleFunction;
	FModifierCompatibilityRule CompatibilityRuleDelegate;

	/** In order to create new instances of this modifier */
	TSubclassOf<UActorModifierCoreBase> Class;

	/** Function to create a new profiler instance for a modifier */
	TFunction<TSharedPtr<FActorModifierCoreProfiler>(UActorModifierCoreBase*)> ProfilerFunction;

#if WITH_EDITOR
	/** Is this modifier visible or hidden in menu */
	bool bHidden;

	/** Display name of this modifier for menus */
	FText DisplayName;

	/** Description of the modifier as tooltip in menus */
	FText Description;

	/** Color visible in modifier tab */
	FLinearColor Color;

	/** Icon visible in menus, details panel and modifier tab */
	FSlateIcon Icon;
#endif
};

/** Modifier stack clone operation */
struct FActorModifierCoreStackCloneOp
{
	/** Modifier in another stack that we want to clone */
	UActorModifierCoreBase* CloneModifier;

	EActorModifierCoreStackPosition ClonePosition = EActorModifierCoreStackPosition::Before;

	/** Context modifier in the stack we want to clone to */
	UActorModifierCoreBase* ClonePositionContext = nullptr;

	FText* FailReason = nullptr;

	TArray<UActorModifierCoreBase*>* AddedDependencies = nullptr;

#if WITH_EDITOR
	bool bShouldTransact = false;
#endif
};

/** Modifier stack insert/add operation */
struct FActorModifierCoreStackInsertOp
{
	FName NewModifierName = NAME_None;

	EActorModifierCoreStackPosition InsertPosition = EActorModifierCoreStackPosition::Before;

	UActorModifierCoreBase* InsertPositionContext = nullptr;

	FText* FailReason = nullptr;

	TArray<UActorModifierCoreBase*>* AddedDependencies = nullptr;

#if WITH_EDITOR
	bool bShouldTransact = false;
#endif
};

/** Modifier stack move operation */
struct FActorModifierCoreStackMoveOp
{
	UActorModifierCoreBase* MoveModifier;

	EActorModifierCoreStackPosition MovePosition = EActorModifierCoreStackPosition::Before;

	UActorModifierCoreBase* MovePositionContext = nullptr;

	FText* FailReason = nullptr;

#if WITH_EDITOR
	bool bShouldTransact = false;
#endif
};

/** Modifier stack remove operation */
struct FActorModifierCoreStackRemoveOp
{
	UActorModifierCoreBase* RemoveModifier;

	bool bRemoveDependencies = false;

	TArray<UActorModifierCoreBase*>* RemovedDependencies = nullptr;

	FText* FailReason = nullptr;

#if WITH_EDITOR
	bool bShouldTransact = false;
#endif
};

/** Modifier stack search operation */
struct FActorModifierCoreStackSearchOp
{
	ACTORMODIFIERCORE_API static const FActorModifierCoreStackSearchOp& GetDefault();

	/** Position to look relative to a context */
	EActorModifierCoreStackPosition Position = EActorModifierCoreStackPosition::After;

	/** Position context to operate search */
	UActorModifierCoreBase* PositionContext = nullptr;

	/** Skip stacks during search to only focus on modifiers */
	bool bSkipStack = true;

	/** Also looks in nested stacks */
	bool bRecurse = true;
};

/** Will lock modifiers execution during lifetime of this struct */
struct ACTORMODIFIERCORE_API FActorModifierCoreScopedLock final
{
	explicit FActorModifierCoreScopedLock(UActorModifierCoreBase* InModifier);
	explicit FActorModifierCoreScopedLock(const TSet<UActorModifierCoreBase*>& InModifiers);

	~FActorModifierCoreScopedLock();

protected:
	TSet<TWeakObjectPtr<UActorModifierCoreBase>> ModifiersWeak;
};

struct FActorModifierCoreStatus
{
	FActorModifierCoreStatus() {}
	explicit FActorModifierCoreStatus(EActorModifierCoreStatus InStatus, const FText& InStatusMessage)
		: Status(InStatus)
		, StatusMessage(InStatusMessage)
	{}

	EActorModifierCoreStatus GetStatus() const
	{
		return Status;
	}

	const FText& GetStatusMessage() const
	{
		return StatusMessage;
	}

protected:
	/** Modifier last status */
	EActorModifierCoreStatus Status = EActorModifierCoreStatus::Success;

	/** Modifier last status message */
	FText StatusMessage = FText::GetEmpty();
};

USTRUCT()
struct FActorModifierCoreExecutionTask
{
	GENERATED_BODY()

	/** Restore original state until first dirty modifier, last -> first */
	void Restore();
	
	/** Build the execution chain of dirty modifiers and execute, first -> last */
	void Apply(TConstArrayView<UActorModifierCoreBase*> InModifiers, UActorModifierCoreStack* InStack);

	void Next();
	void Fail();
	void Skip();

	UActorModifierCoreBase* GetCurrentModifier() const;

	void ReplaceModifier(UActorModifierCoreBase& InOldModifier, UActorModifierCoreBase& InNewModifier);
	void RemoveModifier(UActorModifierCoreBase& InModifier);

private:
	void BuildExecutionChain(TConstArrayView<UActorModifierCoreBase*> InModifiers, UActorModifierCoreStack* InStack);
	void ExecuteCurrentModifier();

	UPROPERTY()
	TObjectPtr<UActorModifierCoreStack> ModifierStack;

	UPROPERTY()
	TArray<TObjectPtr<UActorModifierCoreBase>> Modifiers;

	int32 CurrentIndex = INDEX_NONE;
};