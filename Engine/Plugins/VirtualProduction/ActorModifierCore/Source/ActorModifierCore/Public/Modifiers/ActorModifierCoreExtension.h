// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "UObject/WeakObjectPtr.h"

class UActorModifierCoreBase;
enum class EActorModifierCoreEnableReason : uint8;
enum class EActorModifierCoreDisableReason : uint8;

/**
 * A modifier extension is a piece of logic that multiple modifiers can reuse
 * Can be customized for a specific scenario, only extends one modifier
 * Differs from singleton shared object that can handle or be used by multiple modifiers
 * Will follow the lifecycle of the modifier it is attached to
 */
class FActorModifierCoreExtension : public TSharedFromThis<FActorModifierCoreExtension>
{
	friend class UActorModifierCoreBase;

public:
	ACTORMODIFIERCORE_API virtual ~FActorModifierCoreExtension();

protected:
	/** Overwrite this, called when the extension is created */
	virtual void OnExtensionInitialized() {}
	
	/** Overwrite this, called when the modifier is enabled */
	virtual void OnExtensionEnabled(EActorModifierCoreEnableReason InReason) {}

	/** Overwrite this, called when the modifier is disabled */
	virtual void OnExtensionDisabled(EActorModifierCoreDisableReason InReason) {}

	/** Get the modifier actor */
	ACTORMODIFIERCORE_API AActor* GetModifierActor() const;

	/** Get the modifier actor world */
	ACTORMODIFIERCORE_API UWorld* GetModifierWorld() const;

	/** Get the modifier actor level */
	ACTORMODIFIERCORE_API ULevel* GetModifierLevel() const;

	/** Get the casted modifier to which this extension is linked */
	template <typename InModifierClass, typename = typename TEnableIf<TIsDerivedFrom<InModifierClass, UActorModifierCoreBase>::IsDerived>::Type>
	InModifierClass* GetModifier() const
	{
		return Cast<InModifierClass>(GetModifier());
	}

	/** Get the modifier to which this extension is linked */
	UActorModifierCoreBase* GetModifier() const
	{
		return ModifierWeak.Get();
	}

	/** Get this extension type without needing a cast */
	FName GetExtensionType() const
	{
		return ExtensionType;
	}

	/** Is this extension enabled */
	bool IsExtensionEnabled() const
	{
		return bExtensionEnabled;
	}

private:
	/** INTERNAL USE ONLY, will be called right after the constructor to setup this extension */
	void ConstructInternal(UActorModifierCoreBase* InModifier, const FName& InExtensionType);

	/** INTERNAL USE ONLY, will enable the extension and call virtual functions */
	void EnableExtension(EActorModifierCoreEnableReason InReason);

	/** INTERNAL USE ONLY, will disable the extension and call virtual functions */
	void DisableExtension(EActorModifierCoreDisableReason InReason);

	/** Modifier to which this extension is linked */
	TWeakObjectPtr<UActorModifierCoreBase> ModifierWeak = nullptr;

	/** Extension type to retrieve it on instance without cast needed */
	FName ExtensionType;

	/** Extension was initialized properly and ready to use */
	bool bExtensionInitialized = false;

	/** Extension is enabled */
	bool bExtensionEnabled = false;
};