// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierCoreSharedObject.generated.h"

/**
	Abstract base class for all modifier shared data,
	these will be saved into one single shared actor per world,
	can be queried by modifiers to read and write data,
	create children of this class to share same data across modifiers
*/
UCLASS(NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, Abstract)
class ACTORMODIFIERCORE_API UActorModifierCoreSharedObject : public UObject
{
	GENERATED_BODY()
};