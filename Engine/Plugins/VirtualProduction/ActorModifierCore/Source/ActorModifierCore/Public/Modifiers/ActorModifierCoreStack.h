// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ActorModifierCoreBase.h"
#include "ActorModifierCoreDefs.h"
#include "ActorModifierCoreStack.generated.h"

class UActorModifierCoreBlueprintBase;
class UActorModifierCoreComponent;
class USceneComponent;

/** A modifier stack contains modifiers and is also a modifier by itself */
UCLASS(MinimalAPI, BlueprintType, DefaultToInstanced, EditInlineNew)
class UActorModifierCoreStack : public UActorModifierCoreBase
{
	GENERATED_BODY()

	friend class UActorModifierCoreBase;
	friend class UActorModifierCoreComponent;
	friend class UActorModifierCoreSubsystem;

	friend class FActorModifierCoreEditorDetailCustomization;
	friend class UActorModifierCoreEditorStackCustomization;

	friend struct FActorModifierCoreExecutionTask;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModifierUpdated, UActorModifierCoreBase* /** UpdatedItem */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnModifierAdded, UActorModifierCoreBase* /** Item */, EActorModifierCoreEnableReason /** Reason */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnModifierRemoved, UActorModifierCoreBase* /** Item */, EActorModifierCoreDisableReason /** Reason */)

	static FOnModifierAdded::RegistrationType& OnModifierAdded()
	{
		return OnModifierAddedDelegate;
	}

	static FOnModifierRemoved::RegistrationType& OnModifierRemoved()
	{
		return OnModifierRemovedDelegate;
	}

	static FOnModifierUpdated::RegistrationType& OnModifierMoved()
	{
		return OnModifierMovedDelegate;
	}

	static FOnModifierUpdated::RegistrationType& OnModifierReplaced()
	{
		return OnModifierReplacedDelegate;
	}

	/** Create a new stack by passing the actor and the parent stack if there is one */
	static UActorModifierCoreStack* Create(UActorModifierCoreComponent* InComponent, UActorModifierCoreStack* InParentStack = nullptr);

	/** Returns true if this stack is frozen */
	ACTORMODIFIERCORE_API bool IsModifierStackFrozen() const;

	/**
	 * Set whether the modifier stack is frozen
	 * @see bModifierStackFrozen
	 */
	ACTORMODIFIERCORE_API void SetModifierStackFrozen(bool bInFrozen);

	/** Gets all modifiers in this stack, does not recurse */
	TConstArrayView<UActorModifierCoreBase*> GetModifiers() const
	{
		return Modifiers;
	}

	/** Get modifiers of a specific class only in this stack, does not recurse */
	template <class InModifierType
		UE_REQUIRES(std::is_base_of_v<UActorModifierCoreBase, InModifierType>)>
	void GetClassModifiers(TArray<InModifierType*>& OutModifiers) const
	{
		for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
		{
			if (InModifierType* CastedModifier = Cast<InModifierType>(Modifier))
			{
				OutModifiers.Add(CastedModifier);
			}
		}
	}

	/** Gets the first modifier of a specific class only in this stack, does not recurse */
	template <class InModifierType
		UE_REQUIRES(std::is_base_of_v<UActorModifierCoreBase, InModifierType>)>
	InModifierType* GetClassModifier() const
	{
		for (const TObjectPtr<UActorModifierCoreBase>& Modifier : Modifiers)
		{
			if (InModifierType* CastedModifier = Cast<InModifierType>(Modifier))
			{
				return CastedModifier;
			}
		}
		return nullptr;
	}

	/** Gets the first modifier in this stack, does not recurse */
	ACTORMODIFIERCORE_API UActorModifierCoreBase* GetFirstModifier() const;

	/** Gets the last modifier in this stack, does not recurse */
	ACTORMODIFIERCORE_API UActorModifierCoreBase* GetLastModifier() const;

	/** Gets all modifiers found after this one in the stack that depends on this modifier */
	bool GetDependentModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutDependentModifiers) const;

	/** Gets all modifiers found before this one in the stack that are required by this modifier */
	bool GetRequiredModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutDependentModifiers) const;

	/** Check that we have a modifier inside this stack, checks also nested stacks */
	ACTORMODIFIERCORE_API bool ContainsModifier(const FName& InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;
	ACTORMODIFIERCORE_API bool ContainsModifier(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;
	ACTORMODIFIERCORE_API bool ContainsModifier(const UActorModifierCoreBase* InSearchModifier, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;

	/** Finds a modifier inside this stack, returns first found, checks also nested stacks */
	ACTORMODIFIERCORE_API UActorModifierCoreBase* FindModifier(FName InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;
	ACTORMODIFIERCORE_API UActorModifierCoreBase* FindModifier(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;

	/** Finds modifiers inside this stack, checks also nested stacks */
	ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> FindModifiers(FName InSearchName, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;
	ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> FindModifiers(const UClass* InSearchClass, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const;

	/** This is the root actor stack if we do not have any parent stack */
	ACTORMODIFIERCORE_API bool IsRootStack() const;

	/** Execute those function when the stack is restored, before executing it again */
	ACTORMODIFIERCORE_API void ProcessFunctionOnRestore(const TFunction<void()>& InFunction);

	/** Execute those function when the stack is on idle, done with updates */
	void ProcessFunctionOnIdle(const TFunction<void()>& InFunction);

	/** Process a search function to use before/after position context, stops when false is returned */
	ACTORMODIFIERCORE_API bool ProcessSearchFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions) const;

	/** Does this stack contains any modifiers */
	virtual bool IsModifierEmpty() const override
	{
		return Modifiers.IsEmpty();
	}

	/** Set profiling mode for stack and modifiers inside */
	ACTORMODIFIERCORE_API void SetModifierProfiling(bool bInProfiling);

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Process a function through each modifier in the stack and also the stacks below, stop when we return false */
	virtual bool ProcessFunction(TFunctionRef<bool(const UActorModifierCoreBase*)> InFunction, const FActorModifierCoreStackSearchOp& InSearchOptions = FActorModifierCoreStackSearchOp::GetDefault()) const override;

	/** Checks that we have a modifier with this name inside this stack before another modifier, checks also nested stacks */
	bool ContainsModifierBefore(const FName& InSearchName, const UActorModifierCoreBase* InBeforeModifier) const;
	bool ContainsModifierBefore(const UActorModifierCoreBase* InSearchModifier, const UActorModifierCoreBase* InBeforeModifier) const;

	/** Checks that we have a modifier with this name inside this stack after another modifier, checks also nested stacks */
	bool ContainsModifierAfter(const FName& InSearchName, const UActorModifierCoreBase* InAfterModifier) const;
	bool ContainsModifierAfter(const UActorModifierCoreBase* InSearchModifier, const UActorModifierCoreBase* InAfterModifier) const;

	/** Checks whether all modifier in this stack are initialized */
	bool IsModifierStackInitialized() const;

	/** Clone a modifier with options from another stack/actor, returns the newly inserted modifier, supports BATCH operation */
	UActorModifierCoreBase* CloneModifier(FActorModifierCoreStackCloneOp& InCloneOp);

	/** Insert/Add a modifier with options in the stack, returns the newly inserted modifier, supports BATCH operation */
	UActorModifierCoreBase* InsertModifier(FActorModifierCoreStackInsertOp& InInsertOp);

	/** Moves a modifier with options in the stack, if fail will return a failing reason, supports BATCH operation */
	bool MoveModifier(FActorModifierCoreStackMoveOp& InMoveOp);

	/** Removes a modifier from this stack, supports BATCH operation */
	bool RemoveModifier(FActorModifierCoreStackRemoveOp& InRemoveOp);

	/** Removes all modifiers from this stack in one batch to reduce updates */
	bool RemoveAllModifiers();

	/** Register this stack to the subsystem to query it later only if root stack */
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;

	/** Unregister this stack to the subsystem and propagates to the children modifiers */
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;

	/** Called when the whole stack is disabled, propagates to the children modifiers */
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;

	/** Called when the whole stack is enabled again, propagates to the children modifiers */
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;

	/** A stack will run if all its modifiers are ready to run */
	virtual bool IsModifierReady() const override;

	/** Restore the state before this stack was apply by reversing executed modifiers */
	virtual void RestorePreState() override;

	/** Execute this stack and the modifiers it contains */
	virtual void Apply() override;

	/** Calls OnModifyingActorTransformed of each modifier in the stack if enabled */
	virtual void OnModifiedActorTransformed() override;

	/** Called when a modifier in the stack is dirty */
	virtual void OnModifierDirty(UActorModifierCoreBase* DirtyModifier, bool bExecute) override;

private:
	/** Called when a modifier is added to the stack */
	ACTORMODIFIERCORE_API static FOnModifierAdded OnModifierAddedDelegate;

	/** Called when a modifier is removed from the stack */
	ACTORMODIFIERCORE_API static FOnModifierRemoved OnModifierRemovedDelegate;

	/** Called when a modifier is moved in the stack */
	ACTORMODIFIERCORE_API static FOnModifierUpdated OnModifierMovedDelegate;

	/** Called when a modifier is replaced in the stack (blueprint) */
	ACTORMODIFIERCORE_API static FOnModifierUpdated OnModifierReplacedDelegate;

	/** Sets the stack to receive tick events */
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;

	/** INTERNAL USE ONLY, allows tickable modifier to mark themselves dirty when checking IsModifierDirtyable */
	void TickModifier(float InDelta) const;

	/** Called when the modifier stack frozen has changed */
	void OnModifierStackFrozenChanged();

	/** Called when the actor transform is updated */
	void OnActorTransformUpdated(USceneComponent*, EUpdateTransformFlags, ETeleportType);

	/** Called when the execution task is done */
	void OnModifierExecutionFinished(bool bInResult);

	/** Schedules a modifier optimization on idle */
	void ScheduleModifierOptimization(bool bInInvalidateAll = false);

	/** Checks for any possible modifier optimization within the stack */
	void CheckModifierOptimization(bool bInInvalidateAll);

	/** Replaces blueprint modifier by their new object */
	void OnBlueprintModifierReplaced(UActorModifierCoreBlueprintBase* InOldModifier, UActorModifierCoreBlueprintBase* InNewModifier);

	/** Contains actual modifiers in the stack */
	UPROPERTY(VisibleInstanceOnly, NoClear, Export, Instanced, Category = "Modifiers")
	TArray<TObjectPtr<UActorModifierCoreBase>> Modifiers;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	FActorModifierCoreExecutionTask ExecutionTask;

	/** Used by root stack to trigger itself */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TOptional<FActorModifierCoreExecutionTask> RootExecutionTask;

	/**
	 * EXPERIMENTAL:
	 * Freeze the stack to disable modifiers from being added/removed/moved, while also
	 * preventing restore states from occurring.
	 * The only operation allowed in a frozen state is to remove the stack as a whole.
	 * This can be used to temporarily apply a set of modifiers once, and persist the changes after removing the modifiers.
	 */
	UPROPERTY(EditInstanceOnly, Category="Modifier", meta=(DisplayName="Freeze Stack"))
	bool bModifierStackFrozen = false;

	/** Enable profiling for the modifiers in this stack */
	UPROPERTY(EditInstanceOnly, Category="Modifier", meta=(DisplayName="Enable Profiling"))
	bool bModifierProfiling = false;

	bool bAllModifiersDirty = true;

	/** Functions to execute once when the stack is on idle */
	TArray<TFunction<void()>> OnIdleFunctions;

	/** Functions to execute once when the stack is restored */
	TArray<TFunction<void()>> OnRestoreFunctions;
};
