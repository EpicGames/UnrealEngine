// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UnrealTemplate.h"

#define UE_API ACTORMODIFIERCORE_API

namespace UE::ActorModifierCore
{

enum class ERenderStateDirtyReason : uint8
{
	/** Render state change reason is not known */
	Unknown = 0,
	/** Render state was changed because a primitive became visible/hidden */
	Visibility = 1 << 0,
	/** Render state was changed because a material was changed */
	Material = 1 << 1,
	/** Render state was dirtied because of translucency priority */
	TranslucencyPriority = 1 << 2,
	/** Render state was dirtied because of geometry change */
	Geometry = 1 << 3,
	/** Render state dirtied because of a scene tree change */
	SceneTree = 1 << 4,

	All = 0xFF,
};
ENUM_CLASS_FLAGS(ERenderStateDirtyReason);

/** Get the current render state reason */
UE_API ERenderStateDirtyReason GetRenderStateDirtyReason();

/** 
 * Determines whether the given reason is part of the current render state dirty reason
 * When the current reason is 'unknown' this returns true.
 */
UE_API bool IsRenderStateDirtyRelevant(ERenderStateDirtyReason InReason);

/**
 * Tracks the reason a render state dirty happened.
 * Used to avoid unnecessary computations from modifiers.
 */
struct FRenderStateDirtyReasonScope : public FNoncopyable
{
	UE_API explicit FRenderStateDirtyReasonScope(ERenderStateDirtyReason InReason);
	UE_API ~FRenderStateDirtyReasonScope();

private:
	ERenderStateDirtyReason PreviousReason;
};

} // UE::ActorModifierCore

#undef UE_API
