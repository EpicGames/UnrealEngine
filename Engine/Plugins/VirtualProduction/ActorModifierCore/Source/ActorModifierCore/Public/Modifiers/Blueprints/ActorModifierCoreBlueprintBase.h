// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreBase.h"

#include "ActorModifierCoreBlueprintBase.generated.h"

/** Abstract base class for all blueprint modifier */
UCLASS(MinimalAPI, Blueprintable, Abstract, EditInlineNew, HideCategories=(Tags, AssetUserData, Activation, Collision, Cooking))
class UActorModifierCoreBlueprintBase : public UActorModifierCoreBase
{
	GENERATED_BODY()

	friend class UActorModifierCoreStack;

protected:
	/** Called once to setup modifier metadata */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierSetupEvent(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, FActorModifierCoreMetadata& OutMetadata);

	/** Called when the modifier gets recompiled and replaced in the stack */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierReplacedEvent(AActor* InTargetActor);

	/** Called when this modifier is added in a stack on an actor */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierAddedEvent(AActor* InTargetActor, EActorModifierCoreEnableReason InReason);

	/** Called when this modifier is enabled */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierEnabledEvent(AActor* InTargetActor, EActorModifierCoreEnableReason InReason);

	/** Called when this modifier is disabled */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierDisabledEvent(AActor* InTargetActor, EActorModifierCoreDisableReason InReason);

	/** Called when this modifier is removed from a stack on an actor */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierRemovedEvent(AActor* InTargetActor, EActorModifierCoreDisableReason InReason);

	/** Called before this modifier is applied on an actor to save all relevant state */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierSaveStateEvent(AActor* InTargetActor);

	/** Called to restore this modifier actions on an actor */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers", meta=(ForceAsFunction))
	void OnModifierRestoreStateEvent(AActor* InTargetActor);

	/** Called to apply a custom action on an actor */
	UFUNCTION(BlueprintImplementableEvent, Category="Modifiers")
	bool OnModifierApplyEvent(AActor* InTargetActor, FText& OutFailReason);

	/** Flag this modifier as needing an update after a property value has changed */
	UFUNCTION(BlueprintCallable, Category="Modifiers")
	void FlagModifierDirty();

private:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase
};