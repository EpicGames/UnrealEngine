// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "UObject/Package.h"
#include "ActorModifierCoreGeneratedClass.generated.h"

/** Blueprint modifier class generated by UActorModifierCoreBlueprint */
UCLASS(MinimalAPI)
class UActorModifierCoreGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()
};
