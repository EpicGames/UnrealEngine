// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

class FStructOnScope;
class UActorModifierCoreBase;

/** Profiler class to track modifier usage and performance */
class FActorModifierCoreProfiler : public TSharedFromThis<FActorModifierCoreProfiler>
{
	friend class UActorModifierCoreBase;
	friend struct FActorModifierCoreMetadata;

public:
	static inline const FName ExecutionTimeName = TEXT("ExecutionTime");
	static inline const FName AverageExecutionTimeName = TEXT("AverageExecutionTime");
	static inline const FName TotalExecutionTimeName = TEXT("TotalExecutionTime");
	static inline const FName FrameCountDeltaName = TEXT("FrameCountDelta");
	static inline const FName FrameRateDeltaName = TEXT("FrameRateDelta");

	ACTORMODIFIERCORE_API virtual ~FActorModifierCoreProfiler();

	/** Override this in child classes to setup stats being measured by the profiler */
	ACTORMODIFIERCORE_API virtual void SetupProfilingStats();

	/** Override this in child classes to measure initial stats, called before modifier is applied */
	ACTORMODIFIERCORE_API virtual void BeginProfiling();

	/** Override this in child classes to measure final stats, called after modifier is applied */
	ACTORMODIFIERCORE_API virtual void EndProfiling();

	/** Gets the main stats to display and hide the others */
	ACTORMODIFIERCORE_API virtual TSet<FName> GetMainProfilingStats() const;

	FName GetProfilerType() const
	{
		return ProfilerType;
	}

	FString GetProfilerTag() const
	{
		return ProfilerTag;
	}

	TSharedPtr<FStructOnScope> GetStructProfilerStats() const
	{
		return StructProfilerStats;
	}

	UActorModifierCoreBase* GetModifier() const
	{
		return ModifierWeak.Get();
	}

	template <typename InModifierClass, typename = typename TEnableIf<TIsDerivedFrom<InModifierClass, UActorModifierCoreBase>::IsDerived>::Type>
	InModifierClass* GetModifier() const
	{
		return Cast<InModifierClass>(GetModifier());
	}

	ACTORMODIFIERCORE_API AActor* GetModifierActor() const;

	ACTORMODIFIERCORE_API double GetExecutionTime() const;
	ACTORMODIFIERCORE_API double GetAverageExecutionTime() const;
	ACTORMODIFIERCORE_API double GetTotalExecutionTime() const;
	ACTORMODIFIERCORE_API int64 GetFrameCountDelta() const;
	ACTORMODIFIERCORE_API float GetFrameRateDelta() const;

protected:
	/** INTERNAL USE ONLY, called after this class instance is created */
	void ConstructInternal(UActorModifierCoreBase* InModifier, const FName& InProfilerType);

	/** Used to store and update profiling stats */
	FInstancedPropertyBag ProfilerStats;

private:
	/** Profiled modifier */
	TWeakObjectPtr<UActorModifierCoreBase> ModifierWeak = nullptr;

	/** Type of this profiler object */
	FName ProfilerType;

	/** Used to only read profiling stats */
	TSharedPtr<FStructOnScope> StructProfilerStats = nullptr;

	/** Profiler tag for log */
	FString ProfilerTag;

	float FrameRateStart = 0.f;
	uint64 FrameCountStart = 0;
	uint64 ExecutionTimeStart = 0;
	double TotalExecutionTime = 0;
	uint64 ExecutionCount = 0;
};