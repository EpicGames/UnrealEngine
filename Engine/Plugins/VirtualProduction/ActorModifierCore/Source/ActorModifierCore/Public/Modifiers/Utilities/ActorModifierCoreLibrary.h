// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Modifiers/Utilities/ActorModifierCoreLibraryDefs.h"
#include "ActorModifierCoreLibrary.generated.h"

class UActorModifierCoreBase;
class UActorModifierCoreStack;

/** Blueprint Create/Read/Update/Delete operations for modifiers */
UCLASS(MinimalAPI, DisplayName="Motion Design Modifier Library")
class UActorModifierCoreLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Marks the modifier dirty
	 * @param InModifier The modifier to dirty
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API void MarkModifierDirty(UActorModifierCoreBase* InModifier);

	/**
	 * Retrieves the modifier stack or create one if none is found
	 * @param InActor The actor to get the modifier stack from
	 * @param OutModifierStack [Out] The modifier stack for this actor
	 * @param bInCreateIfNone Whether to create the modifier stack if none is found
	 * @return true when a valid modifier stack is returned
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool FindModifierStack(AActor* InActor, UActorModifierCoreStack*& OutModifierStack, bool bInCreateIfNone = false);

	/**
	 * Creates and insert a new modifier into a modifier stack
	 * @param InModifierStack The modifier stack to use for the operation
	 * @param InOperation The data for this operation
	 * @param OutNewModifier [Out] The newly created modifier
	 * @return true when a valid modifier is returned
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool InsertModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreInsertOperation& InOperation, UActorModifierCoreBase*& OutNewModifier);

	/**
	 * Clones an existing modifier into a modifier stack
	 * @param InModifierStack The modifier stack to use for the operation
	 * @param InOperation The data for this operation
	 * @param OutNewModifier [Out] The newly created modifier
	 * @return true when a valid modifier is returned
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool CloneModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreCloneOperation& InOperation, UActorModifierCoreBase*& OutNewModifier);

	/**
	 * Moves an existing modifier into a modifier stack
	 * @param InModifierStack The modifier stack to use for the operation
	 * @param InOperation The data for this operation
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool MoveModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreMoveOperation& InOperation);

	/**
	 * Removes an existing modifier from a modifier stack
	 * @param InModifierStack The modifier stack to use for the operation
	 * @param InOperation The data for this operation
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool RemoveModifier(UActorModifierCoreStack* InModifierStack, const FActorModifierCoreRemoveOperation& InOperation);

	/**
	 * Sets the state of an existing modifier
	 * @param InModifier The modifier to use for the operation
	 * @param bInState The new state for the modifier
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool EnableModifier(UActorModifierCoreBase* InModifier, bool bInState);

	/**
	 * Checks the state of an existing modifier
	 * @param InModifier The modifier to read from
	 * @param bOutEnabled The modifier enabled state
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool IsModifierEnabled(const UActorModifierCoreBase* InModifier, bool& bOutEnabled);

	/**
	 * Retrieves the modifier stack this modifier is in
	 * @param InModifier The modifier to read from
	 * @param OutModifierStack [Out] The modifier stack this modifier belongs to
	 * @return true when a valid stack is returned
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierStack(const UActorModifierCoreBase* InModifier, UActorModifierCoreStack*& OutModifierStack);

	/**
	 * Retrieves the actor modified by a modifier
	 * @param InModifier The modifier to read from
	 * @param OutModifiedActor [Out] The actor modified by this modifier
	 * @return true when a valid actor is returned
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierActor(const UActorModifierCoreBase* InModifier, AActor*& OutModifiedActor);

	/**
	 * Retrieves the modifier name of an existing modifier
	 * @param InModifier The modifier to read from
	 * @param OutModifierName [Out] The modifier name
	 * @return true when a valid name is returned
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierName(const UActorModifierCoreBase* InModifier, FName& OutModifierName);

	/**
	 * Retrieves the modifier name from a modifier class
	 * @param InModifierClass The modifier class to resolve the name from
	 * @param OutModifierName [Out] The modifier name
	 * @return true when a valid name is returned
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierNameByClass(TSubclassOf<UActorModifierCoreBase> InModifierClass, FName& OutModifierName);

	/**
	 * Retrieves the modifier category of an existing modifier
	 * @param InModifier The modifier to read from
	 * @param OutModifierCategory [Out] The modifier category
	 * @return true when a valid category is returned
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierCategory(const UActorModifierCoreBase* InModifier, FName& OutModifierCategory);

	/**
	 * Retrieves the modifier category from a modifier class
	 * @param InModifierClass The modifier class to resolve the category from
	 * @param OutModifierCategory [Out] The modifier category
	 * @return true when a valid category is returned
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierCategoryByClass(TSubclassOf<UActorModifierCoreBase> InModifierClass, FName& OutModifierCategory);

	/**
	 * Retrieves the modifier categories available
	 * @param OutModifierCategories [Out] The modifier categories registered
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierCategories(TSet<FName>& OutModifierCategories);

	/**
	 * Retrieves the modifiers classes by a category
	 * @param InCategory The modifier category to match
	 * @param OutSupportedModifierClasses [Out] The modifier classes that match the category
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifiersByCategory(FName InCategory, TSet<TSubclassOf<UActorModifierCoreBase>>& OutSupportedModifierClasses);

	/**
	 * Retrieves the modifier class from a modifier name
	 * @param InModifierName The modifier name to resolve
	 * @param OutModifierClass [Out] The modifier class that matches the name
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetModifierClass(FName InModifierName, TSubclassOf<UActorModifierCoreBase>& OutModifierClass);

	/**
	 * Retrieves all modifiers from a modifier stack
	 * @param InModifierStack The modifier stack to read from
	 * @param OutModifiers [Out] The modifiers contained within the stack
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetStackModifiers(const UActorModifierCoreStack* InModifierStack, TArray<UActorModifierCoreBase*>& OutModifiers);

	/**
	 * Retrieves all modifiers found after this one that depends on this modifier
	 * @param InModifier The modifier that is required by others
	 * @param OutModifiers [Out] The dependent modifiers that required the modifier
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetDependentModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutModifiers);

	/**
	 * Retrieves all modifiers found before this one that are required for this modifier
	 * @param InModifier The modifier that requires other modifiers
	 * @param OutModifiers [Out] The required modifiers to use the modifier
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetRequiredModifiers(UActorModifierCoreBase* InModifier, TSet<UActorModifierCoreBase*>& OutModifiers);

	/**
	 * Returns the first modifier of a specified class in the stack
	 * @param InModifierStack The modifier stack to search
	 * @param InModifierClass The class of the modifier to look for
	 * @return the modifier of the specified class, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility", meta=(DeterminesOutputType="InModifierClass"))
	static ACTORMODIFIERCORE_API UActorModifierCoreBase* FindModifierByClass(UActorModifierCoreStack* InModifierStack, TSubclassOf<UActorModifierCoreBase> InModifierClass);

	/**
	 * Returns the first modifier with specified name in the stack
	 * @param InModifierStack The modifier stack to search
	 * @param InModifierName The name of the modifier to look for
	 * @return the modifier with specified name, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API UActorModifierCoreBase* FindModifierByName(const UActorModifierCoreStack* InModifierStack, FName InModifierName);

	/**
	 * Finds all modifiers with specified class in the stack
	 * @param InModifierStack The modifier stack to search
	 * @param InModifierClass The class of the modifier to look for
	 * @return the modifiers with specified class, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility", meta=(DeterminesOutputType="InModifierClass"))
	static ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> FindModifiersByClass(UActorModifierCoreStack* InModifierStack, TSubclassOf<UActorModifierCoreBase> InModifierClass);

	/**
	 * Finds all modifiers with specified class in the stack
	 * @param InModifierStack The modifier stack to search
	 * @param InModifierName The name of the modifier to look for
	 * @return the modifiers with specified name, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> FindModifiersByName(UActorModifierCoreStack* InModifierStack, FName InModifierName);

	/**
	 * Checks if a modifier is contained in the stack
	 * @param InModifierStack The modifier stack to search
	 * @param InModifier The modifier to look for
	 * @return true if the modifier is contained within that stack
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool ContainsModifier(UActorModifierCoreStack* InModifierStack, UActorModifierCoreBase* InModifier);

	/**
	 * Gets all modifier classes supported by this actor at a specific position
	 * @param InActor The actor to check for compatibility
	 * @param OutSupportedModifierClasses [Out] The modifier classes available
	 * @param InContextPosition The context position to insert the modifier
	 * @param InContextModifier The context modifier for insertion
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetSupportedModifiers(AActor* InActor, TSet<TSubclassOf<UActorModifierCoreBase>>& OutSupportedModifierClasses, EActorModifierCoreStackPosition InContextPosition = EActorModifierCoreStackPosition::Before, UActorModifierCoreBase* InContextModifier = nullptr);

	/**
	 * Gets all available modifier classes registered
	 * @param OutAvailableModifierClasses [Out] The modifier classes registered and available to use
	 * @return true when the operation was successful
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API bool GetAvailableModifiers(TSet<TSubclassOf<UActorModifierCoreBase>>& OutAvailableModifierClasses);

	/**
	 * Sets the modifier metadata name
	 * @param InMetadata The modifier metadata to use
	 * @param InName The modifier name to set
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataName(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, FName InName);

	/**
	 * Sets the modifier metadata category
	 * @param InMetadata The modifier metadata to use
	 * @param InCategory The modifier category to set
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataCategory(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, FName InCategory);

	/**
	 * Sets the modifier metadata display name (EDITOR-ONLY)
	 * @param InMetadata The modifier metadata to use
	 * @param InName The modifier name to set
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataDisplayName(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, const FText& InName);

	/**
	 * Sets the modifier metadata color (EDITOR-ONLY)
	 * @param InMetadata The modifier metadata to use
	 * @param InColor The modifier color to set
	 * @return The modifier metadata to chain operations
	 */
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataColor(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, const FLinearColor& InColor);

	/**
	 * Sets the modifier metadata description (EDITOR-ONLY)
	 * @param InMetadata The modifier metadata to use
	 * @param InDescription The modifier description to set
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataDescription(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, const FText& InDescription);

	/**
	 * Adds a modifier metadata dependency for this modifier
	 * @param InMetadata The modifier metadata to use
	 * @param InModifierClass The modifier dependency to add
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& AddModifierMetadataDependency(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, TSubclassOf<UActorModifierCoreBase> InModifierClass);

	/**
	 * Sets the modifier metadata compatibility rule
	 * @param InMetadata The modifier metadata to use
	 * @param InDelegate The modifier rule to set
	 * @return The modifier metadata to chain operations
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Utility")
	static ACTORMODIFIERCORE_API FActorModifierCoreMetadata& SetModifierMetadataCompatibilityRule(UPARAM(Ref) FActorModifierCoreMetadata& InMetadata, const FModifierCompatibilityRule& InDelegate);
};