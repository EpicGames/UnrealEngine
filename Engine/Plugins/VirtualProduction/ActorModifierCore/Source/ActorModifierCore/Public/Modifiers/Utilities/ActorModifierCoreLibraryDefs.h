// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreDefs.h"
#include "ActorModifierCoreLibraryDefs.generated.h"

class UActorModifierCoreBase;

USTRUCT(BlueprintType)
struct FActorModifierCoreInsertOperation
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TSubclassOf<UActorModifierCoreBase> ModifierClass;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	EActorModifierCoreStackPosition InsertPosition = EActorModifierCoreStackPosition::Before;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> InsertPositionContext = nullptr;
};

USTRUCT(BlueprintType)
struct FActorModifierCoreCloneOperation
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> CloneModifier = nullptr;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	EActorModifierCoreStackPosition ClonePosition = EActorModifierCoreStackPosition::Before;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> ClonePositionContext = nullptr;
};

USTRUCT(BlueprintType)
struct FActorModifierCoreMoveOperation
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> MoveModifier = nullptr;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	EActorModifierCoreStackPosition MovePosition = EActorModifierCoreStackPosition::Before;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> MovePositionContext = nullptr;
};

USTRUCT(BlueprintType)
struct FActorModifierCoreRemoveOperation
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	TObjectPtr<UActorModifierCoreBase> RemoveModifier = nullptr;

	UPROPERTY(BlueprintReadWrite, Category="Motion Design|Modifiers|Utility")
	bool bRemoveDependencies = false;
};