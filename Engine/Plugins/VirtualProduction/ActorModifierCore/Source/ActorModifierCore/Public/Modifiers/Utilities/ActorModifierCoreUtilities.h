// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreBase.h"

namespace UE::ActorModifierCore::Utilities
{
	/** Climbs up the attachment tree to find a modifier with a specific class */
	ACTORMODIFIERCORE_API UActorModifierCoreBase* FindFirstActorModifierByClass(const AActor* InStartActor, const TSubclassOf<UActorModifierCoreBase>& InModifierClass);

	/** Safely gets the actor label or name (if label is empty), returning an empty string if actor is null */
	ACTORMODIFIERCORE_API FString GetActorNameSafe(const AActor* InActor);
}
