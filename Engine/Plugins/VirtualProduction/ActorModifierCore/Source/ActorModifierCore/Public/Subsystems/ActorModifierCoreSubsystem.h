// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Modifiers/ActorModifierCoreDefs.h"
#include "Subsystems/EngineSubsystem.h"
#include "Tickable.h"
#include "ActorModifierCoreSubsystem.generated.h"

class AActor;
class AActorModifierCoreSharedActor;
class UClass;
class UActorModifierCoreBase;
class UActorModifierCoreBlueprintBase;
class UActorModifierCoreSharedObject;
class UActorModifierCoreStack;
struct FAssetData;

/** This subsystem handle all modifiers stack active in the engine and allows to create modifiers with registered metadata */
UCLASS()
class UActorModifierCoreSubsystem : public UEngineSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

	friend class UActorModifierCoreEditorSubsystem;
	friend class AActorModifierCoreSharedActor;
	friend class UActorModifierCoreStack;
	friend struct FActorModifierCoreMetadata;

public:
	static constexpr const TCHAR* BlueprintClass = TEXT("/Script/ActorModifierCoreBlueprint.ActorModifierCoreBlueprint");

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModifierClassRegistered, const FActorModifierCoreMetadata& /** ModifierMetadata */)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModifierStackRegistered, const UActorModifierCoreStack* /** ActorRootStack */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnModifierReplaced, UActorModifierCoreBlueprintBase* /** PreviousModifier */, UActorModifierCoreBlueprintBase* /** ReplacementModifier */)

	static FOnModifierClassRegistered::RegistrationType& OnModifierClassRegistered()
	{
		return OnModifierClassRegisteredDelegate;
	}

	static FOnModifierClassRegistered::RegistrationType& OnModifierClassUnregistered()
	{
		return OnModifierClassUnregisteredDelegate;
	}

	static FOnModifierReplaced::RegistrationType& OnModifierReplaced()
	{
		return OnModifierReplacedDelegate;
	}

	ACTORMODIFIERCORE_API static UActorModifierCoreSubsystem* Get();

	/** Gets the ticker for modifiers to use instead of CoreTicker */
	ACTORMODIFIERCORE_API static FTSTicker& GetModifierTicker();

	UActorModifierCoreSubsystem();

	/** Register a modifier class or override an already existing one */
	ACTORMODIFIERCORE_API bool RegisterModifierClass(const UClass* InModifierClass, bool bInOverrideIfExists = false);
	ACTORMODIFIERCORE_API bool UnregisterModifierClass(const FName& InName);
	ACTORMODIFIERCORE_API bool UnregisterModifierClass(const UClass* InModifierClass);
	ACTORMODIFIERCORE_API bool IsRegisteredModifierClass(const FName& InName) const;
	ACTORMODIFIERCORE_API bool IsRegisteredModifierClass(const UClass* InClass) const;

	/** Return the modifier name or none from the modifier class provided */
	ACTORMODIFIERCORE_API FName GetRegisteredModifierName(const UClass* InModifierClass) const;

	/** Return the modifier class from the modifier name provided */
	ACTORMODIFIERCORE_API TSubclassOf<UActorModifierCoreBase> GetRegisteredModifierClass(FName InModifierName) const;

	/** Return a set of class for all modifiers registered */
	TSet<TSubclassOf<UActorModifierCoreBase>> GetRegisteredModifierClasses() const;

	/** Returns the name of modifiers that are currently registered */
	ACTORMODIFIERCORE_API TSet<FName> GetRegisteredModifiers() const;

	/** Returns the modifiers names that are allowed for specific actor/stack and before another modifier */
	ACTORMODIFIERCORE_API TSet<FName> GetAllowedModifiers(AActor* InActor, UActorModifierCoreBase* InContextModifier = nullptr, EActorModifierCoreStackPosition InContextPosition = EActorModifierCoreStackPosition::Before) const;

	/** Returns the modifiers name that match a specific category */
	TSet<FName> GetCategoryModifiers(const FName& InCategory) const;

	/** Returns the categories for all registered modifiers */
	TSet<FName> GetModifierCategories() const;

	/** Returns the category this modifier is in */
	ACTORMODIFIERCORE_API FName GetModifierCategory(const FName& InModifier) const;

#if WITH_EDITOR
	/** Returns the modifiers hidden to the user */
	ACTORMODIFIERCORE_API TSet<FName> GetHiddenModifiers() const;
#endif

	/** Returns modifiers from the stack where we can move the provided modifier, required and dependent modifiers from this MoveModifier will not appear in the list */
	ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> GetAllowedMoveModifiers(UActorModifierCoreBase* InMoveModifier) const;

	/** This is the correct way to retrieve a modifier stack for a specific actor if it has one */
	ACTORMODIFIERCORE_API UActorModifierCoreStack* GetActorModifierStack(const AActor* InActor) const;

	/** This is the correct way to add modifier component to a specific actor if it has none */
	ACTORMODIFIERCORE_API UActorModifierCoreStack* AddActorModifierStack(AActor* InActor) const;

	/** Return the actor this stack is linked to */
	const AActor* GetModifierStackActor(const UActorModifierCoreStack* InStack) const;

	/** Loops through each registered modifier metadata, only used to read */
	ACTORMODIFIERCORE_API bool ForEachModifierMetadata(TFunctionRef<bool(const FActorModifierCoreMetadata&)> InProcessFunction) const;

	/** Process this on matching registered metadata, only used to read */
	ACTORMODIFIERCORE_API bool ProcessModifierMetadata(const FName& InName, TFunctionRef<bool(const FActorModifierCoreMetadata&)> InProcessFunction) const;

	/** Get modifier shared object from a specific level and class, only one or none can exists per level */
	UActorModifierCoreSharedObject* GetModifierSharedObject(ULevel* InLevel, TSubclassOf<UActorModifierCoreSharedObject> InClass, bool bInCreateIfNone = false) const;

	/** Enable or disable modifiers, will update the original array and perform a transaction if wanted */
	ACTORMODIFIERCORE_API bool EnableModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, bool bInEnabled, bool bInShouldTransact = false) const;

	/** Add a modifier to actors, adds a stack automatically if none is found, will update the original array and perform a transaction if wanted */
	ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> AddActorsModifiers(const TSet<AActor*>& InActors, FActorModifierCoreStackInsertOp& InAddOp) const;

	/** Remove all modifiers from actors, will update the original array and perform a transaction if wanted */
	ACTORMODIFIERCORE_API bool RemoveActorsModifiers(const TSet<AActor*>& InActors, bool bInShouldTransact = false) const;

	/** Remove modifiers from different actors or stacks, will update the original array and perform a transaction if wanted */
	ACTORMODIFIERCORE_API bool RemoveModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, FActorModifierCoreStackRemoveOp& InRemoveOp) const;

	/** Removes modifier components and modifier stacks */
	ACTORMODIFIERCORE_API bool RemoveModifierStacks(const TSet<UActorModifierCoreStack*>& InStacks, bool bInShouldTransact = false) const;

	/** Insert a modifier in a stack before or after another modifier, will perform a transaction if wanted */
	ACTORMODIFIERCORE_API UActorModifierCoreBase* InsertModifier(UActorModifierCoreStack* InStack, FActorModifierCoreStackInsertOp& InInsertOp) const;

	/** Moves a modifier in a stack before or after another modifier, will perform a transaction if wanted */
	ACTORMODIFIERCORE_API bool MoveModifier(UActorModifierCoreStack* InStack, FActorModifierCoreStackMoveOp& InMoveOp) const;

	/** Move modifiers to a specific target modifier position, will perform menu action transaction if wanted */
	ACTORMODIFIERCORE_API bool MoveModifiers(const TArray<UActorModifierCoreBase*>& InModifiers, UActorModifierCoreStack* InStack, FActorModifierCoreStackMoveOp& InMoveOp) const;

	/** Clone modifiers to a specific target modifier position, will perform menu action transaction if wanted */
	ACTORMODIFIERCORE_API TArray<UActorModifierCoreBase*> CloneModifiers(const TArray<UActorModifierCoreBase*>& InModifiers, UActorModifierCoreStack* InStack, FActorModifierCoreStackCloneOp& InCloneOp) const;

	/** Get order dependent modifiers in the correct order for stack operations */
	ACTORMODIFIERCORE_API void GetSortedModifiers(const TSet<UActorModifierCoreBase*>& InModifiers, AActor* InTargetActor, UActorModifierCoreBase* InTargetModifier, EActorModifierCoreStackPosition InPosition, TArray<UActorModifierCoreBase*>& OutMoveModifiers, TArray<UActorModifierCoreBase*>& OutCloneModifiers) const;

protected:
	/** Called when a modifier class is registered */
	ACTORMODIFIERCORE_API static FOnModifierClassRegistered OnModifierClassRegisteredDelegate;

	/** Called when a modifier class is unregistered */
	ACTORMODIFIERCORE_API static FOnModifierClassRegistered OnModifierClassUnregisteredDelegate;

	/** Called when an actor modifier stack is registered */
	static FOnModifierStackRegistered OnModifierStackRegisteredDelegate;

	/** Called when an actor modifier stack is unregistered */
	static FOnModifierStackRegistered OnModifierStackUnregisteredDelegate;

	/** Called when a blueprint modifier gets replaced */
	static FOnModifierReplaced OnModifierReplacedDelegate;

	/** Ticker for modifiers to use instead of CoreTicker */
	static FTSTicker ModifierTicker;

	//~ Begin FTickableObjectBase
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableObjectBase

	//~ Begin UEngineSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End UEngineSubsystem

	void OnInsertModifier(const FActorModifierCoreStackInsertOp& InInsertOp) const;

	/** Register a root actor stack to query this actor stack from everywhere, is called automatically at creation or deserialization by stack */
	bool RegisterActorModifierStack(UActorModifierCoreStack* InStack);

	/** Unregister a root stack by providing the actor it is attached to */
	bool UnregisterActorModifierStack(const AActor* InActor);

	/** Unregister a root stack from an actor by providing the stack itself */
	bool UnregisterActorModifierStack(const UActorModifierCoreStack* InStack);

	/** Do we have a root modifier stack for this actor */
	bool IsRegisteredActorModifierStack(const AActor* InActor) const;

	/** Do we have a registered root stack */
	bool IsRegisteredActorModifierStack(const UActorModifierCoreStack* InStack) const;

	/** Checks whether we can add a modifier to the stack */
	bool ValidateModifierCreation(const FName& InName, const UActorModifierCoreStack* InStack, FText& OutFailReason, UActorModifierCoreBase* InBeforeModifier = nullptr) const;

	/** Creates a modifier based on a name and a stack, should be called by the stack itself */
	UActorModifierCoreBase* CreateModifierInstance(const FName& InName, UActorModifierCoreStack* InStack, FText& OutFailReason, UActorModifierCoreBase* InBeforeModifier = nullptr) const;

	/** Builds a modifier dependency list, does not make additional checks */
	bool BuildModifierDependencies(const FName& InName, TArray<FName>& OutBeforeModifiers) const;

	/** Done only once at subsystem initialization */
	void ScanForModifiers();

	void OnAssetRegistryFilesLoaded();
	void OnAssetRegistryAssetAdded(const FAssetData& InAssetData);
	void OnAssetRegistryAssetUpdated(const FAssetData& InAssetData);
	void OnAssetRegistryAssetRemoved(const FAssetData& InAssetData);

	void OnBlueprintObjectsReplaced(const TMap<UObject*, UObject*>& InReplacements);

	void RegisterModifierAsset(const FAssetData& InAssetData);
	void UnregisterModifierAsset(const FAssetData& InAssetData);

	/** Registers a modifier shared provider actor for a world */
	bool RegisterModifierSharedProvider(AActorModifierCoreSharedActor* InSharedActor) const;

	/** Unregisters a modifier shared provider actor for a world */
	bool UnregisterModifierSharedProvider(const AActor* InSharedActor) const;

	/** Gets a modifier shared provider for a world, creates one if none is found */
	AActorModifierCoreSharedActor* GetModifierSharedProvider(ULevel* InLevel, bool bInSpawnIfNotFound = true) const;

	/** stores modifiers metadata, you can override these if you want a different behaviour */
	TMap<FName, TSharedRef<FActorModifierCoreMetadata>> ModifiersMetadata;

	/** stores modifiers stacks per actor, there should be only one root stack per actor */
	TMap<TWeakObjectPtr<const AActor>, TWeakObjectPtr<UActorModifierCoreStack>> ModifierStacks;

	/** Stores modifiers providers per world/level, there should be only one provider per level */
	TMap<TWeakObjectPtr<const ULevel>, TWeakObjectPtr<AActorModifierCoreSharedActor>> ModifierSharedProviders;

	/** Asset Registry files loaded */
	bool bFilesLoaded = false;
};
