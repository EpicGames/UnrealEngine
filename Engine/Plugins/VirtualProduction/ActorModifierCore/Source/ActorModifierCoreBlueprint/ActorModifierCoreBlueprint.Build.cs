// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierCoreBlueprint : ModuleRules
{
    public ActorModifierCoreBlueprint(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
        
        PublicDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "BlueprintGraph",
		        "Core",
		        "CoreUObject",
		        "Engine",
		        "SlateCore"
	        }
        );
        
        PrivateDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "ActorModifierCore"
	        }
        );
    }
}