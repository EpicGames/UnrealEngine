// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierCoreBlueprint.h"

#include "Modifiers/Blueprints/ActorModifierCoreGeneratedClass.h"

#if WITH_EDITOR
UClass* UActorModifierCoreBlueprint::GetBlueprintClass() const
{
	return UActorModifierCoreGeneratedClass::StaticClass();
}

void UActorModifierCoreBlueprint::GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const
{
	OutAllowedChildrenOfClasses.Add(UActorModifierCoreGeneratedClass::StaticClass());
}

bool UActorModifierCoreBlueprint::SupportedByDefaultBlueprintFactory() const
{
	return false;
}
#endif
