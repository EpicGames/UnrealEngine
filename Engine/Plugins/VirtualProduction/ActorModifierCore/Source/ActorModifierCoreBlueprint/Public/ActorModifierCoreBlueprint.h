// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "ActorModifierCoreBlueprint.generated.h"

/**
 * Blueprint object that defines a user modifier
 */
UCLASS(MinimalAPI)
class UActorModifierCoreBlueprint : public UBlueprint
{
	GENERATED_BODY()

	//~ Begin UBlueprint
#if WITH_EDITOR
	virtual UClass* GetBlueprintClass() const override;
	virtual void GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const override;
	virtual bool SupportedByDefaultBlueprintFactory() const override;
#endif
	//~ End UBlueprint
};
