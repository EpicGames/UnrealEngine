// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierCoreEditor : ModuleRules
{
	public ActorModifierCoreEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"Core",
			}
		);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"ActorModifierCoreBlueprint",
				"ApplicationCore",
				"AssetDefinition",
				"CoreUObject",
				"CustomDetailsView",
				"EditorSubsystem",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"OperatorStackEditor",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
