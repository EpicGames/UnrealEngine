// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierCoreEditorModule.h"

#include "ActorModifierCoreBlueprint.h"
#include "KismetCompilerModule.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "Modifiers/Customizations/ActorModifierCoreEditorDetailCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

void FActorModifierCoreEditorModule::StartupModule()
{
	RegisterDetailCustomizations();
	RegisterBlueprintCustomizations();
}

void FActorModifierCoreEditorModule::ShutdownModule()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		UnregisterDetailCustomizations();
	}
}

void FActorModifierCoreEditorModule::RegisterDetailCustomizations()
{
	// Register custom layouts
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UActorModifierCoreStack::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FActorModifierCoreEditorDetailCustomization::MakeInstance));
}

void FActorModifierCoreEditorModule::UnregisterDetailCustomizations()
{
	// Unregister custom layouts
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName CustomizationName : CustomizationNames)
		{
			PropertyModule->UnregisterCustomClassLayout(CustomizationName);
		}

		CustomizationNames.Empty();
	}
}

void FActorModifierCoreEditorModule::RegisterBlueprintCustomizations()
{
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.OverrideBPTypeForClass(UActorModifierCoreBlueprintBase::StaticClass(), UActorModifierCoreBlueprint::StaticClass());
}

IMPLEMENT_MODULE(FActorModifierCoreEditorModule, ActorModifierCoreEditor)
