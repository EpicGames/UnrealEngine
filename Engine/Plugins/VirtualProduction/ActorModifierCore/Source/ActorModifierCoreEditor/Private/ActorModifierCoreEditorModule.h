// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Containers/Array.h"
#include "UObject/NameTypes.h"

class FActorModifierCoreEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

protected:
	void RegisterDetailCustomizations();
	void UnregisterDetailCustomizations();

	void RegisterBlueprintCustomizations();

	TArray<FName> CustomizationNames;
};