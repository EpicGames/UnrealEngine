// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierCoreEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FActorModifierCoreEditorStyle::FActorModifierCoreEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon16x16(16.f, 16.f);
	
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	/** Modifiers */
	Set("ClassIcon.ActorModifierCoreBase",                     new IMAGE_BRUSH_SVG("Icons/BaseModifier", Icon16x16));
	Set("ClassIcon.ActorModifierCoreComponent",                new IMAGE_BRUSH_SVG("Icons/BaseModifier", Icon16x16));
	Set("ClassIcon.ActorModifierCoreEditorStackCustomization", new IMAGE_BRUSH_SVG("Icons/BaseModifier", Icon16x16));
	Set("Profiling",                                           new IMAGE_BRUSH_SVG("Icons/Profiling",    Icon16x16));
	Set("FreezeStack",                                         new IMAGE_BRUSH("Icons/FreezeStack",      Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FActorModifierCoreEditorStyle::~FActorModifierCoreEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}