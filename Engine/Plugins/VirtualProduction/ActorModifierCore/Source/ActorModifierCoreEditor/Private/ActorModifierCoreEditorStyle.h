// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FActorModifierCoreEditorStyle : public FSlateStyleSet
{
public:
	FActorModifierCoreEditorStyle();
	
	virtual ~FActorModifierCoreEditorStyle() override;
	
	static FActorModifierCoreEditorStyle& Get()
	{
		static FActorModifierCoreEditorStyle StyleSet;
		return StyleSet;
	}
};