// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreEditorMenu.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreEditorMenuContext"

void UE::ActorModifierCoreEditor::OnExtendAddModifierMenu(UToolMenu* InAddToolMenu, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UActorModifierCoreEditorSubsystem* ModifierEditorSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!ModifierSubsystem
		|| !ModifierEditorSubsystem
		|| !InAddToolMenu)
	{
		return;
	}

	TSet<FName> ShowModifiers = ModifierSubsystem->GetRegisteredModifiers();
	ShowModifiers = ShowModifiers.Difference(ModifierSubsystem->GetHiddenModifiers());
	for (AActor* Actor : InData.Context.GetContextActors())
	{
		ShowModifiers = ShowModifiers.Intersect(ModifierSubsystem->GetAllowedModifiers(Actor));
	}

	if (ShowModifiers.IsEmpty())
	{
		return;
	}

	for (const FName& Modifier : ShowModifiers)
	{
		const FName ModifierCategory = ModifierSubsystem->GetModifierCategory(Modifier);
		TSharedPtr<const FActorModifierCoreEditorCategoryMetadata> Category = ModifierEditorSubsystem->FindModifierCategory(ModifierCategory);
		const FText ModifierDisplayCategory = Category ? Category->DisplayName : FText::FromName(ModifierCategory);

		FText ModifierDisplayName = FText::FromName(Modifier);
		FText ModifierDescription = FText::GetEmpty();
		FSlateIcon ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		ModifierSubsystem->ProcessModifierMetadata(Modifier, [&ModifierDisplayName, &ModifierIcon, &ModifierDescription](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			ModifierDisplayName = InMetadata.GetDisplayName();
			ModifierIcon = InMetadata.GetIcon();
			ModifierDescription = InMetadata.GetDescription();
			return true;
		});

		const FName ModifierCategorySection("ContextModifierCategory" + ModifierCategory.ToString());
		FToolMenuSection* CategorySection = InAddToolMenu->FindSection(ModifierCategorySection);
		if (!CategorySection)
		{
			CategorySection = &InAddToolMenu->AddSection(
				ModifierCategorySection,
				ModifierDisplayCategory,
				FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
		}

		const FText Label = FText::Format(
			LOCTEXT("AddModifier.Label", "{0}"),
			ModifierDisplayName);

		const FText Tooltip = FText::Format(
			LOCTEXT("AddModifier.Tooltip", "Add Modifier {0} to the currently selected actors\n{1}"),
			ModifierDisplayName,
			ModifierDescription);

		const FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnAddModifierMenuAction, Modifier, InData));

		const FName MenuEntryName(TEXT("AddModifier") + Modifier.ToString());
		FToolMenuEntry AddModifierEntry = FToolMenuEntry::InitMenuEntry(
			MenuEntryName,
			Label,
			Tooltip,
			ModifierIcon,
			Action);

		CategorySection->AddEntry(AddModifierEntry);
	}
}

void UE::ActorModifierCoreEditor::OnExtendInsertModifierMenu(UToolMenu* InInsertToolMenu, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UActorModifierCoreEditorSubsystem* ModifierEditorSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!ModifierSubsystem
		|| !ModifierEditorSubsystem
		|| !InInsertToolMenu)
	{
		return;
	}

	if (InData.Context.GetContextModifiers().IsEmpty() || InData.Context.GetContextModifiers().Num() > 1)
	{
		return;
	}

	UActorModifierCoreBase* ContextModifier = InData.Context.GetContextModifiers().Array()[0];

	if (!ContextModifier)
	{
		return;
	}

	const EActorModifierCoreStackPosition InsertPosition = InData.Options.GetMenuType() == EActorModifierCoreEditorMenuType::InsertBefore ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;
	TSet<FName> ShowModifiers = ModifierSubsystem->GetAllowedModifiers(ContextModifier->GetModifiedActor(), ContextModifier, InsertPosition);
	ShowModifiers = ShowModifiers.Difference(ModifierSubsystem->GetHiddenModifiers());
	if (ShowModifiers.IsEmpty())
	{
		return;
	}

	for (const FName& Modifier : ShowModifiers)
	{
		const FName ModifierCategory = ModifierSubsystem->GetModifierCategory(Modifier);
		TSharedPtr<const FActorModifierCoreEditorCategoryMetadata> Category = ModifierEditorSubsystem->FindModifierCategory(ModifierCategory);
		const FText ModifierDisplayCategory = Category ? Category->DisplayName : FText::FromName(ModifierCategory);

		FText ModifierDisplayName = FText::FromName(Modifier);
		FText ModifierDescription = FText::GetEmpty();
		FSlateIcon ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		ModifierSubsystem->ProcessModifierMetadata(Modifier, [&ModifierDisplayName, &ModifierIcon, &ModifierDescription](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			ModifierDisplayName = InMetadata.GetDisplayName();
			ModifierIcon = InMetadata.GetIcon();
			ModifierDescription = InMetadata.GetDescription();
			return true;
		});

		const FName ModifierCategorySection("ContextModifierCategory" + ModifierCategory.ToString());
		FToolMenuSection* CategorySection = InInsertToolMenu->FindSection(ModifierCategorySection);
		if (!CategorySection)
		{
			CategorySection = &InInsertToolMenu->AddSection(
				ModifierCategorySection,
				ModifierDisplayCategory,
				FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
		}

		const FText Label = FText::Format(
			LOCTEXT("InsertModifier.Label", "{0}"),
			ModifierDisplayName);

		const FText Tooltip = FText::Format(
			LOCTEXT("InsertModifier.Tooltip", "Insert Modifier {0} {1} current selection\n{2}"),
			ModifierDisplayName,
			FText::FromString(InsertPosition == EActorModifierCoreStackPosition::Before ? TEXT("before") : TEXT("after")),
			ModifierDescription);

		const FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnInsertModifierMenuAction, ContextModifier, Modifier, InData));

		const FName MenuEntryName(TEXT("InsertModifier") + Modifier.ToString());
		FToolMenuEntry InsertModifierEntry = FToolMenuEntry::InitMenuEntry(
			MenuEntryName,
			Label,
			Tooltip,
			ModifierIcon,
			Action);

		CategorySection->AddEntry(InsertModifierEntry);
	}
}

void UE::ActorModifierCoreEditor::OnExtendRemoveModifierMenu(UToolMenu* InRemoveToolMenu, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UActorModifierCoreEditorSubsystem* ModifierEditorSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!ModifierSubsystem
		|| !ModifierEditorSubsystem
		|| !InRemoveToolMenu)
	{
		return;
	}

	if (InData.Context.IsEmpty())
	{
		return;
	}

	// remove all stack modifiers
	if (InData.Context.ContainsAnyActor())
	{
		static const FText RemoveAllLabel(LOCTEXT("RemoveAllModifiers.Label", "Remove all modifiers"));
		static const FText RemoveAllTooltip(LOCTEXT("RemoveAllModifiers.Tooltip", "Remove all modifiers from the currently selected actors"));
		static const FName RemoveAllEntryName(TEXT("RemoveAllModifiers"));
		static const FSlateIcon RemoveAllIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		const FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnRemoveActorModifierMenuAction, InData));

		FToolMenuEntry RemoveAllModifierEntry = FToolMenuEntry::InitMenuEntry(
			RemoveAllEntryName,
			RemoveAllLabel,
			RemoveAllTooltip,
			RemoveAllIcon,
			Action);

		InRemoveToolMenu->AddMenuEntry(RemoveAllEntryName, RemoveAllModifierEntry);
	}

	// remove all selected modifiers
	if (InData.Context.ContainsAnyModifier())
	{
		static const FText RemoveSelectedLabel(LOCTEXT("RemoveSelectedModifiers.Label", "Remove selected modifiers"));
		static const FText RemoveSelectedTooltip(LOCTEXT("RemoveSelectedModifiers.Tooltip", "Remove selected modifiers only"));
		static const FName RemoveSelectedEntryName(TEXT("RemoveSelectedModifiers"));
		static const FSlateIcon RemoveSelectedIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		const FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnRemoveModifierMenuAction, InData));

		FToolMenuEntry RemoveSelectedModifierEntry = FToolMenuEntry::InitMenuEntry(
			RemoveSelectedEntryName,
			RemoveSelectedLabel,
			RemoveSelectedTooltip,
			RemoveSelectedIcon,
			Action);

		InRemoveToolMenu->AddMenuEntry(RemoveSelectedEntryName, RemoveSelectedModifierEntry);
	}

	// only handle one actor selected case after this
	if (!InData.Context.ContainsAnyActor() || InData.Context.GetContextActors().Num() > 1)
	{
		return;
	}

	// get current actor stack
	TWeakObjectPtr<AActor> ContextActor = InData.Context.GetContextActors().Array()[0];
	const UActorModifierCoreStack* ModifierStack = ModifierSubsystem->GetActorModifierStack(ContextActor.Get());

	if (!ModifierStack)
	{
		return;
	}

	TConstArrayView<UActorModifierCoreBase*> StackModifiers = ModifierStack->GetModifiers();

	if (StackModifiers.IsEmpty())
	{
		return;
	}

	// add separator
	static const FName SeparatorName("RemoveAllSeparator");
	FToolMenuEntry Separator = FToolMenuEntry::InitSeparator(SeparatorName);
	InRemoveToolMenu->AddMenuEntry(SeparatorName, Separator);

	for (UActorModifierCoreBase* StackModifier : StackModifiers)
	{
		if (!IsValid(StackModifier))
		{
			continue;
		}

		const FName ModifierName = StackModifier->GetModifierName();
		FText ModifierDisplayName = FText::FromName(ModifierName);
		FSlateIcon ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		ModifierSubsystem->ProcessModifierMetadata(ModifierName, [&ModifierDisplayName, &ModifierIcon](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			ModifierDisplayName = InMetadata.GetDisplayName();
			ModifierIcon = InMetadata.GetIcon();
			return true;
		});

		const FText Label = FText::Format(LOCTEXT("RemoveModifier.Label", "{0}"),
				ModifierDisplayName);
		const FText Tooltip = FText::Format(LOCTEXT("RemoveModifier.Tooltip", "Remove Modifier {0} from the currently selected actor"),
				ModifierDisplayName);
		const FName MenuEntryName(TEXT("RemoveModifier") + ModifierName.ToString());

		FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnRemoveSingleModifierMenuAction, StackModifier, InData));

		FToolMenuEntry RemoveModifierEntry = FToolMenuEntry::InitMenuEntry(
			MenuEntryName,
			Label,
			Tooltip,
			ModifierIcon,
			Action);

		InRemoveToolMenu->AddMenuEntry(MenuEntryName, RemoveModifierEntry);
	}
}

void UE::ActorModifierCoreEditor::OnExtendMoveModifierMenu(UToolMenu* InMoveToolMenu, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const UActorModifierCoreEditorSubsystem* ModifierEditorSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!ModifierSubsystem
		|| !ModifierEditorSubsystem
		|| !InMoveToolMenu)
	{
		return;
	}

	if (InData.Context.GetContextModifiers().IsEmpty() || InData.Context.GetContextModifiers().Num() > 1)
	{
		return;
	}

	UActorModifierCoreBase* MoveModifier = InData.Context.GetContextModifiers().Array()[0];

	if (!MoveModifier)
	{
		return;
	}

	TArray<UActorModifierCoreBase*> StackModifiers = ModifierSubsystem->GetAllowedMoveModifiers(MoveModifier);

	if (StackModifiers.IsEmpty())
	{
		return;
	}

	const FName& MoveModifierName = MoveModifier->GetModifierName();

	// move in the beginning
	if (StackModifiers.Contains(MoveModifier->GetRootModifierStack()->GetFirstModifier()))
	{
		static const FText StartLabel = LOCTEXT("MoveModifierStart.Label", "At the start");
		static const FText StartTooltip = LOCTEXT("MoveModifierStart.Tooltip", "Move Modifier at the beginning of the stack");
		static const FName StartMenuEntryName(TEXT("MoveModifierStart"));

		UActorModifierCoreBase* StartPositionModifier = nullptr;
		FUIAction StartAction = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnMoveModifierMenuAction, MoveModifier, StartPositionModifier, InData));

		FToolMenuEntry MoveModifierEntry = FToolMenuEntry::InitMenuEntry(
			StartMenuEntryName,
			StartLabel,
			StartTooltip,
			FSlateIcon(),
			StartAction);

		InMoveToolMenu->AddMenuEntry(StartMenuEntryName, MoveModifierEntry);
	}

	for (UActorModifierCoreBase* PositionModifier : StackModifiers)
	{
		if (!IsValid(PositionModifier))
		{
			continue;
		}

		if (MoveModifier == PositionModifier)
		{
			continue;
		}

		const FName ModifierName = PositionModifier->GetModifierName();
		FText ModifierDisplayName = FText::FromName(ModifierName);
		FSlateIcon ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());

		ModifierSubsystem->ProcessModifierMetadata(ModifierName, [&ModifierDisplayName, &ModifierIcon](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			ModifierDisplayName = InMetadata.GetDisplayName();
			ModifierIcon = InMetadata.GetIcon();
			return true;
		});

		const FText Label = FText::Format(LOCTEXT("MoveModifier.Label", "After {0}"),
				ModifierDisplayName);
		const FText Tooltip = FText::Format(LOCTEXT("MoveModifier.Tooltip", "Move Modifier {0} in the stack"),
				FText::FromName(MoveModifierName));
		const FName MenuEntryName(TEXT("MoveModifier") + ModifierName.ToString());

		FUIAction Action = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnMoveModifierMenuAction, MoveModifier, PositionModifier, InData));

		FToolMenuEntry MoveModifierEntry = FToolMenuEntry::InitMenuEntry(
			MenuEntryName,
			Label,
			Tooltip,
			ModifierIcon,
			Action);

		InMoveToolMenu->AddMenuEntry(MenuEntryName, MoveModifierEntry);
	}
}

void UE::ActorModifierCoreEditor::OnExtendEnableModifierMenu(UToolMenu* InToolMenu, const FActorModifierCoreEditorMenuData& InData)
{
	const bool bEnableMenu = InData.Options.GetMenuType() == EActorModifierCoreEditorMenuType::Enable;

	if (bEnableMenu)
	{
		if (InData.Context.ContainsDisabledStack())
		{
			static const FText EnableStackLabel = LOCTEXT("EnableStack.Label", "Enable selected stack");
			static const FText EnableStackTooltip = LOCTEXT("EnableStack.Tooltip", "Enable selected modifier stack");
			static const FName EnableStackEntryName(TEXT("EnableStack"));

			const FUIAction EnableStackAction = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnEnableModifierMenuAction, InData, true, true));

			const FToolMenuEntry EnableStackEntry = FToolMenuEntry::InitMenuEntry(
				EnableStackEntryName,
				EnableStackLabel,
				EnableStackTooltip,
				FSlateIcon(),
				EnableStackAction);

			InToolMenu->AddMenuEntry(EnableStackEntryName, EnableStackEntry);
		}

		if (InData.Context.ContainsDisabledModifier())
		{
			static const FText EnableModifierLabel = LOCTEXT("EnableModifier.Label", "Enable selected modifier");
			static const FText EnableModifierTooltip = LOCTEXT("EnableModifier.Tooltip", "Enable selected modifier");
			static const FName EnableModifierEntryName(TEXT("EnableModifier"));

			const FUIAction EnableModifierAction = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnEnableModifierMenuAction, InData, true, false));

			const FToolMenuEntry EnableModifierEntry = FToolMenuEntry::InitMenuEntry(
				EnableModifierEntryName,
				EnableModifierLabel,
				EnableModifierTooltip,
				FSlateIcon(),
				EnableModifierAction);

			InToolMenu->AddMenuEntry(EnableModifierEntryName, EnableModifierEntry);
		}
	}
	else
	{
		if (InData.Context.ContainsEnabledStack())
		{
			static const FText DisableStackLabel = LOCTEXT("DisableStack.Label", "Disable selected stack");
			static const FText DisableStackTooltip = LOCTEXT("DisableStack.Tooltip", "Disable selected modifier stack");
			static const FName DisableStackEntryName(TEXT("DisableStack"));

			const FUIAction DisableStackAction = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnEnableModifierMenuAction, InData, false, true));

			const FToolMenuEntry DisableStackEntry = FToolMenuEntry::InitMenuEntry(
				DisableStackEntryName,
				DisableStackLabel,
				DisableStackTooltip,
				FSlateIcon(),
				DisableStackAction);

			InToolMenu->AddMenuEntry(DisableStackEntryName, DisableStackEntry);
		}

		if (InData.Context.ContainsEnabledModifier())
		{
			static const FText DisableModifierLabel = LOCTEXT("DisableModifier.Label", "Disable selected modifier");
			static const FText DisableModifierTooltip = LOCTEXT("DisableModifier.Tooltip", "Disable selected modifier");
			static const FName DisableModifierEntryName(TEXT("DisableModifier"));

			const FUIAction DisableModifierAction = FUIAction(FExecuteAction::CreateLambda(&UE::ActorModifierCoreEditor::OnEnableModifierMenuAction, InData, false, false));

			const FToolMenuEntry DisableModifierEntry = FToolMenuEntry::InitMenuEntry(
				DisableModifierEntryName,
				DisableModifierLabel,
				DisableModifierTooltip,
				FSlateIcon(),
				DisableModifierAction);

			InToolMenu->AddMenuEntry(DisableModifierEntryName, DisableModifierEntry);
		}
	}
}

void UE::ActorModifierCoreEditor::OnAddModifierMenuAction(const FName& InModifier, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	TSet<AActor*> Actors = InData.Context.GetContextActors();
	if (Actors.IsEmpty())
	{
		return;
	}

	FText OutFailReason;
	FActorModifierCoreStackInsertOp InsertOp;
	InsertOp.bShouldTransact = true;
	InsertOp.FailReason = &OutFailReason;
	InsertOp.NewModifierName = InModifier;

	ModifierSubsystem->AddActorsModifiers(Actors, InsertOp);

	if (!OutFailReason.IsEmpty())
	{
		if (InData.Options.ShouldFireNotification())
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnInsertModifierMenuAction(UActorModifierCoreBase* InModifier, const FName& InModifierName, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	UActorModifierCoreStack* ModifierStack = InModifier->GetModifierStack();

	FText OutFailReason;

	FActorModifierCoreStackInsertOp InsertOp;
	InsertOp.bShouldTransact = true;
	InsertOp.FailReason = &OutFailReason;
	InsertOp.InsertPosition = InData.Options.GetMenuType() == EActorModifierCoreEditorMenuType::InsertBefore ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;
	InsertOp.InsertPositionContext = InModifier;
	InsertOp.NewModifierName = InModifierName;

	if (!ModifierSubsystem->InsertModifier(ModifierStack, InsertOp))
	{
		if (InData.Options.ShouldFireNotification())
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnRemoveActorModifierMenuAction(const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	const TSet<AActor*> Actors = InData.Context.GetContextActors();
	if (Actors.IsEmpty())
	{
		return;
	}

	if (!ModifierSubsystem->RemoveActorsModifiers(Actors, true))
	{
		if (InData.Options.ShouldFireNotification())
		{
			static const FText RemoveAllModifiersWarning(LOCTEXT("RemoveAllModifiers", "Could not remove all modifiers from this actor"));
			FNotificationInfo NotificationInfo(RemoveAllModifiersWarning);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnRemoveModifierMenuAction(const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	TSet<UActorModifierCoreBase*> Modifiers = InData.Context.GetContextModifiers();
	if (Modifiers.IsEmpty())
	{
		return;
	}

	FText OutFailReason;
	FActorModifierCoreStackRemoveOp RemoveOp;
	RemoveOp.bShouldTransact = true;
	RemoveOp.FailReason = &OutFailReason;

	if (!ModifierSubsystem->RemoveModifiers(Modifiers, RemoveOp))
	{
		if (InData.Options.ShouldFireNotification())
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnRemoveSingleModifierMenuAction(UActorModifierCoreBase* InModifier, const FActorModifierCoreEditorMenuData& InData)
{
	if (!IsValid(InModifier))
	{
		return;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	FText OutFailReason;
	TSet<UActorModifierCoreBase*> RemoveModifiers { InModifier };

	FActorModifierCoreStackRemoveOp RemoveOp;
	RemoveOp.bShouldTransact = true;
	RemoveOp.FailReason = &OutFailReason;

	if (!ModifierSubsystem->RemoveModifiers(RemoveModifiers, RemoveOp))
	{
		if (InData.Options.ShouldFireNotification())
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
            FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnMoveModifierMenuAction(UActorModifierCoreBase* InMoveModifier, UActorModifierCoreBase* InPositionModifier, const FActorModifierCoreEditorMenuData& InData)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem) || !IsValid(InMoveModifier))
	{
		return;
	}

	FText OutFailReason;

	FActorModifierCoreStackMoveOp MoveOp;
	MoveOp.bShouldTransact = true;
	MoveOp.FailReason = &OutFailReason;
	MoveOp.MoveModifier = InMoveModifier;
	MoveOp.MovePosition = EActorModifierCoreStackPosition::After;
	MoveOp.MovePositionContext = InPositionModifier;

	if (!ModifierSubsystem->MoveModifier(InMoveModifier->GetModifierStack(), MoveOp))
	{
		if (InData.Options.ShouldFireNotification())
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void UE::ActorModifierCoreEditor::OnEnableModifierMenuAction(const FActorModifierCoreEditorMenuData& InData, bool bInEnable, bool bInStack)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	TSet<UActorModifierCoreBase*> Modifiers;
	if (bInStack)
	{
		Modifiers = InData.Context.GetContextStacks();
	}
	else
	{
		Modifiers = InData.Context.GetContextModifiers();
	}

	if (Modifiers.IsEmpty())
	{
		return;
	}

	if (!ModifierSubsystem->EnableModifiers(Modifiers, bInEnable, true))
	{
		if (InData.Options.ShouldFireNotification())
		{
			static const FText EnableStacksWarning(LOCTEXT("EnableStacks", "Could not update stack(s) state"));
			static const FText EnableModifiersWarning(LOCTEXT("EnableModifiers", "Could not update modifier(s) state"));

			FNotificationInfo NotificationInfo(bInStack ? EnableStacksWarning : EnableModifiersWarning);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

#undef LOCTEXT_NAMESPACE