// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreEditorMenuDefs.h"
#include "ToolMenus.h"

namespace UE::ActorModifierCoreEditor
{
	/** For add submenu */
	void OnExtendAddModifierMenu(UToolMenu* InAddToolMenu, const FActorModifierCoreEditorMenuData& InData);

	/** For insert submenu */
	void OnExtendInsertModifierMenu(UToolMenu* InInsertToolMenu, const FActorModifierCoreEditorMenuData& InData);

	/** For remove submenu */
	void OnExtendRemoveModifierMenu(UToolMenu* InRemoveToolMenu, const FActorModifierCoreEditorMenuData& InData);

	/** For move submenu */
	void OnExtendMoveModifierMenu(UToolMenu* InMoveToolMenu, const FActorModifierCoreEditorMenuData& InData);

	/** For enable/disable submenu */
	void OnExtendEnableModifierMenu(UToolMenu* InToolMenu, const FActorModifierCoreEditorMenuData& InData);

	/** Add a modifier action */
	void OnAddModifierMenuAction(const FName& InModifier, const FActorModifierCoreEditorMenuData& InData);

	/** Insert a modifier action */
	void OnInsertModifierMenuAction(UActorModifierCoreBase* InModifier, const FName& InModifierName, const FActorModifierCoreEditorMenuData& InData);

	/** Remove actor modifier action */
	void OnRemoveActorModifierMenuAction(const FActorModifierCoreEditorMenuData& InData);

	/** Remove context modifier action */
	void OnRemoveModifierMenuAction(const FActorModifierCoreEditorMenuData& InData);

	/** Remove single modifier action */
	void OnRemoveSingleModifierMenuAction(UActorModifierCoreBase* InModifier, const FActorModifierCoreEditorMenuData& InData);

	/** Move a modifier action */
	void OnMoveModifierMenuAction(UActorModifierCoreBase* InMoveModifier, UActorModifierCoreBase* InPositionModifier, const FActorModifierCoreEditorMenuData& InData);

	/** Enable/disable modifier action */
	void OnEnableModifierMenuAction(const FActorModifierCoreEditorMenuData& InData, bool bInEnable, bool bInStack);
}