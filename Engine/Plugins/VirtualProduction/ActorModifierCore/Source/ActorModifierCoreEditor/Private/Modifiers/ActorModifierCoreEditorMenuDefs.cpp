// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierCoreEditorMenuDefs.h"

#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

FActorModifierCoreEditorMenuOptions& FActorModifierCoreEditorMenuOptions::CreateSubMenu(bool bInCreateSubMenu)
{
	bCreateSubMenu = bInCreateSubMenu;
	return *this;
}

FActorModifierCoreEditorMenuOptions& FActorModifierCoreEditorMenuOptions::UseTransact(bool bInUseTransact)
{
	bUseTransact = bInUseTransact;
	return *this;
}

FActorModifierCoreEditorMenuOptions& FActorModifierCoreEditorMenuOptions::FireNotification(bool bInFireNotification)
{
	bFireNotification = bInFireNotification;
	return *this;
}

FActorModifierCoreEditorMenuContext::FActorModifierCoreEditorMenuContext(const TSet<TWeakObjectPtr<UObject>>& InContextObjects)
{
	static const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	
	ContextActors.Empty();
	ContextModifiers.Empty();
	ContextStacks.Empty();
	
	for (const TWeakObjectPtr<UObject>& Object : InContextObjects)
	{
		if (Object.IsValid())
		{
			if (Object->IsA<AActor>())
			{
				AActor* Actor = Cast<AActor>(Object.Get());
				ContextActors.Add(Actor);
				if (ModifierSubsystem)
				{
					if (UActorModifierCoreStack* Stack = ModifierSubsystem->GetActorModifierStack(Actor))
					{
						ContextStacks.Add(Stack);
					}
				}
			}
			else if (Object->IsA<UActorModifierCoreStack>())
			{
				UActorModifierCoreStack* Stack = Cast<UActorModifierCoreStack>(Object.Get());
				ContextStacks.Add(Stack);
				if (AActor* Actor = Stack->GetModifiedActor())
				{
					ContextActors.Add(Actor);
				}
			}
			else if (Object->IsA<UActorModifierCoreBase>())
			{
				ContextModifiers.Add(Cast<UActorModifierCoreBase>(Object.Get()));
			}
		}
	}
}

bool FActorModifierCoreEditorMenuContext::IsEmpty() const
{
	return ContextActors.IsEmpty() && ContextModifiers.IsEmpty() && ContextStacks.IsEmpty();
}

bool FActorModifierCoreEditorMenuContext::ContainsAnyActor() const
{
	return !ContextActors.IsEmpty();
}

bool FActorModifierCoreEditorMenuContext::ContainsAnyModifier() const
{
	return !ContextModifiers.IsEmpty();
}

bool FActorModifierCoreEditorMenuContext::ContainsAnyStack() const
{
	return !ContextStacks.IsEmpty();
}

bool FActorModifierCoreEditorMenuContext::ContainsOnlyModifier() const
{
	return ContainsAnyModifier() && ContextActors.IsEmpty() && ContextStacks.IsEmpty();
}

bool FActorModifierCoreEditorMenuContext::ContainsNonEmptyStack() const
{
	for (const UActorModifierCoreBase* Stack : ContextStacks)
	{
		if (Stack && !Stack->IsModifierEmpty())
		{
			return true;
		}
	}
	return false;
}

bool FActorModifierCoreEditorMenuContext::ContainsDisabledModifier() const
{
	for (const UActorModifierCoreBase* Modifier : ContextModifiers)
	{
		if (Modifier && !Modifier->IsModifierEnabled())
		{
			return true;
		}
	}
	return false;
}

bool FActorModifierCoreEditorMenuContext::ContainsDisabledStack() const
{
	for (const UActorModifierCoreBase* Stack : ContextStacks)
	{
		if (Stack && !Stack->IsModifierEnabled())
		{
			return true;
		}
	}
	return false;
}

bool FActorModifierCoreEditorMenuContext::ContainsEnabledModifier() const
{
	for (const UActorModifierCoreBase* Modifier : ContextModifiers)
	{
		if (Modifier && Modifier->IsModifierEnabled())
		{
			return true;
		}
	}
	return false;
}

bool FActorModifierCoreEditorMenuContext::ContainsEnabledStack() const
{
	for (const UActorModifierCoreBase* Stack : ContextStacks)
	{
		if (Stack && Stack->IsModifierEnabled())
		{
			return true;
		}
	}
	return false;
}
