// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierCoreBlueprintFactory.h"

#include "ActorModifierCoreBlueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"
#include "Modifiers/Blueprints/ActorModifierCoreBlueprintBase.h"
#include "Modifiers/Blueprints/ActorModifierCoreGeneratedClass.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreBlueprintFactory"

UActorModifierCoreBlueprintFactory::UActorModifierCoreBlueprintFactory()
{
	ParentClass = UActorModifierCoreBlueprintBase::StaticClass();
	SupportedClass = UActorModifierCoreBlueprint::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UActorModifierCoreBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext)
{
	check(InClass && InClass->IsChildOf<UActorModifierCoreBlueprint>());

	if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf<UActorModifierCoreBlueprintBase>())
	{
		FMessageDialog::Open(EAppMsgType::Ok
			, FText::Format(LOCTEXT("InvalidParentClassMessage", "Unable to create Modifier Blueprint with parent class '{0}'.")
			, FText::FromString(GetNameSafe(ParentClass))));
		return nullptr;
	}

	// Create Blueprint
	UActorModifierCoreBlueprint* Blueprint = CastChecked<UActorModifierCoreBlueprint>(FKismetEditorUtilities::CreateBlueprint(
		ParentClass
		, InParent
		, InName
		, EBlueprintType::BPTYPE_Normal
		, UActorModifierCoreBlueprint::StaticClass()
		, UActorModifierCoreGeneratedClass::StaticClass()
		, InCallingContext)
	);

	return Blueprint;
}

#undef LOCTEXT_NAMESPACE
