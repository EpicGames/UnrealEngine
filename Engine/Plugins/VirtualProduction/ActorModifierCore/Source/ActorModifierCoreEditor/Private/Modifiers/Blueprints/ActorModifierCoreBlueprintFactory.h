// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Templates/SubclassOf.h"
#include "ActorModifierCoreBlueprintFactory.generated.h"

class UActorModifierCoreBlueprintBase;

/** Factory responsible to create a UActorModifierCoreBlueprint object */
UCLASS()
class UActorModifierCoreBlueprintFactory : public UFactory
{
	GENERATED_BODY()

public:
	UActorModifierCoreBlueprintFactory();

	//~ Begin UFactory
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext) override;
	//~ End UFactory

	UPROPERTY()
	TSubclassOf<UActorModifierCoreBlueprintBase> ParentClass;
};
