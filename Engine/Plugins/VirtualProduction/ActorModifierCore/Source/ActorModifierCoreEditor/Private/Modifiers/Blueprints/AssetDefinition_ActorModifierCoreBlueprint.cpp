// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ActorModifierCoreBlueprint.h"

#include "ActorModifierCoreBlueprint.h"
#include "BlueprintEditorModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_ActorModifierCoreBlueprint"

FText UAssetDefinition_ActorModifierCoreBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Actor Modifier Blueprint");
}

FLinearColor UAssetDefinition_ActorModifierCoreBlueprint::GetAssetColor() const
{
	return FLinearColor::White;
}

TSoftClassPtr<UObject> UAssetDefinition_ActorModifierCoreBlueprint::GetAssetClass() const
{
	return UActorModifierCoreBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ActorModifierCoreBlueprint::GetAssetCategories() const
{
	return MakeArrayView(&EAssetCategoryPaths::Blueprint, 1);
}

EAssetCommandResult UAssetDefinition_ActorModifierCoreBlueprint::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	for (UActorModifierCoreBlueprint* Blueprint : InOpenArgs.LoadObjects<UActorModifierCoreBlueprint>())
	{
		bool bShouldOpen = true;

		if (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass)
		{
			bShouldOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo
				, LOCTEXT("InvalidBlueprintClassPrompt",
					"Blueprint could not be loaded because it derives from an invalid class.\n"
					"Check to make sure the parent class for this blueprint hasn't been removed!\n"
					"Do you want to continue (it can crash the editor)?"));
		}

		if (bShouldOpen)
		{
			FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
			TSharedRef<IBlueprintEditor> NewKismetEditor = BlueprintEditorModule.CreateBlueprintEditor(InOpenArgs.GetToolkitMode(), InOpenArgs.ToolkitHost, Blueprint, FBlueprintEditorUtils::ShouldOpenWithDataOnlyEditor(Blueprint));
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
