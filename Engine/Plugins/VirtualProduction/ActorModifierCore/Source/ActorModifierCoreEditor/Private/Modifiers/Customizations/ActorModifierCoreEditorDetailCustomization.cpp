// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Customizations/ActorModifierCoreEditorDetailCustomization.h"

#include "Contexts/OperatorStackEditorContext.h"
#include "DetailLayoutBuilder.h"
#include "Items/OperatorStackEditorItem.h"
#include "Items/OperatorStackEditorObjectItem.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"
#include "Widgets/SOperatorStackEditorWidget.h"

void FActorModifierCoreEditorDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	UOperatorStackEditorSubsystem* OperatorStackSubsystem = UOperatorStackEditorSubsystem::Get();

	const TSharedPtr<IPropertyHandle> ModifiersPropertyHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, Modifiers),
		UActorModifierCoreStack::StaticClass()
	);

	const TSharedPtr<IPropertyHandle> EnablePropertyHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UActorModifierCoreBase, bModifierEnabled),
		UActorModifierCoreBase::StaticClass()
	);

	const TSharedPtr<IPropertyHandle> FreezePropertyHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierStackFrozen),
		UActorModifierCoreStack::StaticClass()
	);

	const TSharedPtr<IPropertyHandle> ProfilingPropertyHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierProfiling),
		UActorModifierCoreStack::StaticClass()
	);

	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = nullptr;
	if (TSharedPtr<IDetailsView> DetailsView = InDetailBuilder.GetDetailsViewSharedPtr())
	{
		KeyframeHandler = DetailsView->GetKeyframeHandler();
	}
	
	if (!ModifiersPropertyHandle.IsValid()
		|| !EnablePropertyHandle.IsValid()
		|| !FreezePropertyHandle.IsValid()
		|| !ProfilingPropertyHandle.IsValid()
		|| !KeyframeHandler.IsValid()
		|| !OperatorStackSubsystem)
	{
		return;
	}

	InDetailBuilder.HideProperty(EnablePropertyHandle);
	InDetailBuilder.HideProperty(FreezePropertyHandle);
	InDetailBuilder.HideProperty(ProfilingPropertyHandle);
	
	IDetailPropertyRow* ModifiersRow = InDetailBuilder.EditDefaultProperty(ModifiersPropertyHandle);

	TArray<TWeakObjectPtr<UObject>> CustomizedObjectsWeak;
	InDetailBuilder.GetObjectsBeingCustomized(CustomizedObjectsWeak);

	TArray<FOperatorStackEditorItemPtr> CustomizedItems;
	Algo::Transform(CustomizedObjectsWeak, CustomizedItems, [](const TWeakObjectPtr<UObject>& InObjectWeak)
	{
		return MakeShared<FOperatorStackEditorObjectItem>(InObjectWeak.Get());
	});
	
	const FOperatorStackEditorContext Context(CustomizedItems);
	const TSharedRef<SOperatorStackEditorWidget> OperatorStackWidget = OperatorStackSubsystem->GenerateWidget();
	OperatorStackWidget->SetKeyframeHandler(KeyframeHandler);
	OperatorStackWidget->SetContext(Context);

	// Do not allow to switch, only modifiers view should be active, hide other choices
	OperatorStackWidget->SetActiveCustomization(TEXT("Modifiers"));
	OperatorStackWidget->SetToolbarVisibility(false);

	ModifiersRow->CustomWidget()
	.ShouldAutoExpand(true)
	[
		OperatorStackWidget
	];
}
