// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"

class FDetailColumnSizeData;

/** Stack details customization in the details panel */
class FActorModifierCoreEditorDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FActorModifierCoreEditorDetailCustomization>();
	}
	
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization
};