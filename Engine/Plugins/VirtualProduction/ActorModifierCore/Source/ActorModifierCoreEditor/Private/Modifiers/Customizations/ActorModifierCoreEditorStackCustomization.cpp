// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Customizations/ActorModifierCoreEditorStackCustomization.h"

#include "ActorModifierCoreEditorStyle.h"
#include "Contexts/OperatorStackEditorMenuContext.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Items/OperatorStackEditorGroupItem.h"
#include "Items/OperatorStackEditorObjectItem.h"
#include "JsonObjectConverter.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreComponent.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreEditorStackCustomization"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreEditorStackCustomization, Log, All);

namespace UE::ActorModifierCoreEditor::Private
{

FText GetStackPropertyTooltip(FName InPropertyName)
{
	const FProperty* Property = FindFProperty<FProperty>(UActorModifierCoreStack::StaticClass(), InPropertyName);
	return Property ? Property->GetToolTipText() : FText::GetEmpty();
}

} // UE::ActorModifierCoreEditor::Private

UActorModifierCoreEditorStackCustomization::UActorModifierCoreEditorStackCustomization()
	: UOperatorStackEditorStackCustomization(
		TEXT("Modifiers")
		, LOCTEXT("CustomizationLabel", "Modifiers")
		, 1
	)
{
	// for stack and modifiers
	RegisterCustomizationFor(UActorModifierCoreBase::StaticClass());

	// Modifiers delegates
	UActorModifierCoreStack::OnModifierAdded().AddUObject(this, &UActorModifierCoreEditorStackCustomization::OnModifierAdded);
	UActorModifierCoreStack::OnModifierMoved().AddUObject(this, &UActorModifierCoreEditorStackCustomization::OnModifierUpdated);
	UActorModifierCoreStack::OnModifierRemoved().AddUObject(this, &UActorModifierCoreEditorStackCustomization::OnModifierRemoved);
	UActorModifierCoreStack::OnModifierReplaced().AddUObject(this, &UActorModifierCoreEditorStackCustomization::OnModifierUpdated);
}

UActorModifierCoreEditorStackCustomization::~UActorModifierCoreEditorStackCustomization()
{
	UActorModifierCoreStack::OnModifierAdded().RemoveAll(this);
	UActorModifierCoreStack::OnModifierMoved().RemoveAll(this);
	UActorModifierCoreStack::OnModifierRemoved().RemoveAll(this);
	UActorModifierCoreStack::OnModifierReplaced().RemoveAll(this);
}

bool UActorModifierCoreEditorStackCustomization::GetRootItem(const FOperatorStackEditorContext& InContext, FOperatorStackEditorItemPtr& OutRootItem) const
{
	TArray<FOperatorStackEditorItemPtr> RootItems;

	// Gather all modifiers stack as root items
	for (const FOperatorStackEditorItemPtr& Item : InContext.GetItems())
	{
		if (!Item.IsValid())
		{
			continue;
		}

		if (Item->IsA<AActor>())
		{
			const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

			for (const AActor* Actor : Item->GetAsArray<AActor>())
			{
				if (UActorModifierCoreStack* Stack = ModifierSubsystem->GetActorModifierStack(Actor))
				{
					RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Stack));
				}
			}
		}
		else if (Item->IsA<UActorModifierCoreComponent>())
		{
			for (const UActorModifierCoreComponent* Component : Item->GetAsArray<UActorModifierCoreComponent>())
			{
				RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Component->GetModifierStack()));
			}
		}
		else if (Item->IsA<UActorModifierCoreStack>())
		{
			for (UActorModifierCoreStack* ModifierStack : Item->GetAsArray<UActorModifierCoreStack>())
			{
				RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(ModifierStack));
			}
		}
		else if (Item->IsA<UActorModifierCoreBase>())
		{
			for (const UActorModifierCoreBase* Modifier : Item->GetAsArray<UActorModifierCoreBase>())
			{
				RootItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Modifier->GetRootModifierStack()));
			}
		}
	}

	OutRootItem = MakeShared<FOperatorStackEditorGroupItem>(RootItems, FOperatorStackEditorItemType(UActorModifierCoreStack::StaticClass(), EOperatorStackEditorItemType::Object));

	return Super::GetRootItem(InContext, OutRootItem);
}

bool UActorModifierCoreEditorStackCustomization::GetChildrenItem(const FOperatorStackEditorItemPtr& InItem, TArray<FOperatorStackEditorItemPtr>& OutChildrenItems) const
{
	if (InItem->IsA<UActorModifierCoreStack>())
	{
		if (InItem->GetValueCount() > 1)
		{
			TMap<UClass*, int32> ClassToIndex;
			TArray<TArray<FOperatorStackEditorItemPtr>> Modifiers;

			for (const UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
			{
				for (UActorModifierCoreBase* Modifier : ModifierStack->GetModifiers())
				{
					if (!IsValid(Modifier))
					{
						continue;
					}

					if (const int32* Index = ClassToIndex.Find(Modifier->GetClass()))
					{
						Modifiers[*Index].Add(MakeShared<FOperatorStackEditorObjectItem>(Modifier));
					}
					else
					{
						TArray<FOperatorStackEditorItemPtr> ModifierGroup;
						ModifierGroup.Add(MakeShared<FOperatorStackEditorObjectItem>(Modifier));
						int32 GroupIndex = Modifiers.Add(ModifierGroup);
						ClassToIndex.Add(Modifier->GetClass(), GroupIndex);
					}
				}
			}

			for (int32 Index = 0; Index < Modifiers.Num(); Index++)
			{
				const UClass* const* ModifierClass = ClassToIndex.FindKey(Index);
				OutChildrenItems.Add(MakeShared<FOperatorStackEditorGroupItem>(Modifiers[Index], FOperatorStackEditorItemType(*ModifierClass, EOperatorStackEditorItemType::Object)));
			}
		}
		else
		{
			for (UActorModifierCoreBase* Modifier : InItem->Get<UActorModifierCoreStack>(0)->GetModifiers())
			{
				if (IsValid(Modifier))
				{
					OutChildrenItems.Add(MakeShared<FOperatorStackEditorObjectItem>(Modifier));
				}
			}
		}
	}

	return Super::GetChildrenItem(InItem, OutChildrenItems);
}

void UActorModifierCoreEditorStackCustomization::CustomizeStackHeader(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder)
{
	if (!InItemTree.GetContext().GetItems().IsEmpty())
	{
		static const FName AddModifierMenuName = TEXT("AddModifierMenu");
		if (!UToolMenus::Get()->IsMenuRegistered(AddModifierMenuName))
		{
			UToolMenu* const AddModifierMenu = UToolMenus::Get()->RegisterMenu(AddModifierMenuName, NAME_None, EMultiBoxType::Menu);
			AddModifierMenu->AddDynamicSection(TEXT("PopulateAddModifierMenu"), FNewToolMenuDelegate::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::FillStackHeaderMenu));
		}

		// Pin used categories
		TSet<FString> PinnedKeywords;
		for (FOperatorStackEditorItemPtr Item : InItemTree.GetAllItems())
		{
			if (Item.IsValid() && Item->IsA<UActorModifierCoreBase>())
			{
				const UActorModifierCoreBase* Modifier = Item->Get<UActorModifierCoreBase>(0);

				if (Modifier && !Modifier->IsModifierStack())
				{
					PinnedKeywords.Add(Modifier->GetModifierCategory().ToString());
				}
			}
		}

		InHeaderBuilder
			.SetToolMenu(
				AddModifierMenuName
				, LOCTEXT("AddModifiersMenu", "Add Modifiers")
				, FAppStyle::GetBrush("Icons.Plus")
			)
			.SetSearchAllowed(true)
			.SetSearchPinnedKeywords(PinnedKeywords);
	}

	Super::CustomizeStackHeader(InItemTree, InHeaderBuilder);
}

void UActorModifierCoreEditorStackCustomization::CustomizeItemHeader(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	// Customize stack and modifier header
	if (InItem->IsA<UActorModifierCoreBase>() && InItem->HasValue(0))
	{
		FBoolProperty* ModifierEnableProperty = FindFProperty<FBoolProperty>(UActorModifierCoreBase::StaticClass(), GET_MEMBER_NAME_CHECKED(UActorModifierCoreBase, bModifierEnabled));

		// Commands for item on key events
		const TSharedPtr<FUICommandList> Commands = CreateModifierCommands(InItem);

		// Action menu available in header in slim toolbar
		static const FName HeaderModifierMenuName = TEXT("HeaderModifierMenu");
		if (!UToolMenus::Get()->IsMenuRegistered(HeaderModifierMenuName))
		{
			UToolMenu* const HeaderModifierMenu = UToolMenus::Get()->RegisterMenu(HeaderModifierMenuName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
			HeaderModifierMenu->AddDynamicSection(TEXT("FillHeaderModifierMenu"), FNewToolMenuDelegate::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::FillItemHeaderActionMenu));
		}

		// Context menu available when right clicking on item
		static const FName ContextModifierMenuName = TEXT("ContextModifierMenu");
		if (!UToolMenus::Get()->IsMenuRegistered(ContextModifierMenuName))
		{
			UToolMenu* const ContextModifierMenu = UToolMenus::Get()->RegisterMenu(ContextModifierMenuName, NAME_None, EMultiBoxType::Menu);
			ContextModifierMenu->AddDynamicSection(TEXT("FillContextModifierMenu"), FNewToolMenuDelegate::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::FillItemContextActionMenu));
		}

		// Item keyword for search
		UActorModifierCoreBase* Modifier = InItem->Get<UActorModifierCoreBase>(0);

		TSet<FString> SearchKeywords
		{
			Modifier->GetModifierName().ToString(),
			Modifier->GetModifierCategory().ToString()
		};

		FSlateIcon ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());
		FLinearColor ModifierColor = FLinearColor::Transparent;
		FText ModifierTooltip = FText::GetEmpty();

		ModifierSubsystem->ProcessModifierMetadata(Modifier->GetModifierName(), [&SearchKeywords, &ModifierIcon, &ModifierColor, &ModifierTooltip](const FActorModifierCoreMetadata& InMetadata)
		{
			ModifierIcon = InMetadata.GetIcon();
			ModifierColor = InMetadata.GetColor();
			ModifierTooltip = InMetadata.GetDescription();
			SearchKeywords.Add(InMetadata.GetDisplayName().ToString());
			return true;
		});

		const bool bIsStack = InItem->IsA<UActorModifierCoreStack>();

		/** Show last execution error messages if failed execution */
		TAttribute<EOperatorStackEditorMessageType> MessageType = EOperatorStackEditorMessageType::None;
		TAttribute<FText> MessageText = FText::GetEmpty();
		FText HeaderLabel = bIsStack
			? LOCTEXT("ModifierStackHeaderLabel", "Modifiers")
			: FText::FromName(Modifier->GetModifierName());

		ModifierSubsystem->ProcessModifierMetadata(
			Modifier->GetModifierName(),
			[&HeaderLabel](const FActorModifierCoreMetadata& InMetadata)->bool
			{
				HeaderLabel = InMetadata.GetDisplayName();
				return true;
			}
		);

		if (InItem->GetValueCount() > 1)
		{
			HeaderLabel = FText::Format(LOCTEXT("ModifiersHeaderLabel", "{0} ({1})"), HeaderLabel, FText::AsNumber(InItem->GetValueCount()));

			if (bIsStack)
			{
				MessageType = EOperatorStackEditorMessageType::Info;
				MessageText = LOCTEXT("MultiModifierView", "You are viewing multiple items");
			}
		}
		else if (!bIsStack)
		{
			TWeakObjectPtr<UActorModifierCoreBase> ModifierWeak(Modifier);

			MessageType = TAttribute<EOperatorStackEditorMessageType>::CreateLambda([ModifierWeak]()
			{
				if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
				{
					if (Modifier->GetModifierLastStatus().GetStatus() == EActorModifierCoreStatus::Warning)
					{
						return EOperatorStackEditorMessageType::Warning;
					}

					if (Modifier->GetModifierLastStatus().GetStatus() == EActorModifierCoreStatus::Error)
					{
						return EOperatorStackEditorMessageType::Error;
					}

					if (!Modifier->IsModifierEnabled())
					{
						return EOperatorStackEditorMessageType::Warning;
					}
				}

				return EOperatorStackEditorMessageType::None;
			});

			MessageText = TAttribute<FText>::CreateLambda([ModifierWeak]()
			{
				if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
				{
					if (!Modifier->IsModifierEnabled())
					{
						return LOCTEXT("ModifierDisabled", "Modifier disabled");
					}

					return Modifier->GetModifierLastStatus().GetStatusMessage();
				}

				return FText::GetEmpty();
			});
		}

		InHeaderBuilder
			.SetSearchAllowed(true)
			.SetSearchKeywords(SearchKeywords)
			.SetExpandable(!bIsStack)
			.SetIcon(ModifierIcon.GetIcon())
			.SetLabel(HeaderLabel)
			.SetTooltip(ModifierTooltip)
			.SetBorderColor(ModifierColor)
			.SetProperty(ModifierEnableProperty)
			.SetCommandList(Commands)
			.SetToolbarMenu(HeaderModifierMenuName)
			.SetContextMenu(ContextModifierMenuName)
			.SetMessageBox(MessageType, MessageText);
	}

	Super::CustomizeItemHeader(InItem, InItemTree, InHeaderBuilder);
}

void UActorModifierCoreEditorStackCustomization::CustomizeItemBody(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InBodyBuilder)
{
	// Customize stack and modifier body
	if (InItem->IsA<UActorModifierCoreBase>())
	{
		FBoolProperty* ModifierEnableProperty = FindFProperty<FBoolProperty>(UActorModifierCoreBase::StaticClass(), GET_MEMBER_NAME_CHECKED(UActorModifierCoreBase, bModifierEnabled));
		FBoolProperty* StackFrozenProperty = FindFProperty<FBoolProperty>(UActorModifierCoreStack::StaticClass(), GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierStackFrozen));
		FBoolProperty* ProfilingEnableProperty = FindFProperty<FBoolProperty>(UActorModifierCoreStack::StaticClass(), GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierProfiling));

		const bool bIsStack = InItem->IsA<UActorModifierCoreStack>();

		InBodyBuilder
			.SetShowDetailsView(!bIsStack)
			.DisallowProperty(ModifierEnableProperty)
			.DisallowProperty(StackFrozenProperty)
			.DisallowProperty(ProfilingEnableProperty);
	}

	Super::CustomizeItemBody(InItem, InItemTree, InBodyBuilder);
}

void UActorModifierCoreEditorStackCustomization::CustomizeItemFooter(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorFooterBuilder& InFooterBuilder)
{
	// Customize stack and modifier footer
	if (InItemTree.GetRootItem()->GetValueCount() == 1 && InItem->IsA<UActorModifierCoreBase>())
	{
		const UActorModifierCoreBase* Modifier = InItem->Get<UActorModifierCoreBase>(0);
		const TSharedPtr<FActorModifierCoreProfiler> Profiler = Modifier->GetProfiler();
		UActorModifierCoreEditorSubsystem* ExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();

		if (Profiler.IsValid())
		{
			const TSharedPtr<SWidget> Widget = ExtensionSubsystem->CreateProfilerWidget(Profiler);

			InFooterBuilder
				.SetCustomWidget(Widget);
		}
	}

	Super::CustomizeItemFooter(InItem, InItemTree, InFooterBuilder);
}

bool UActorModifierCoreEditorStackCustomization::OnIsItemDraggable(const FOperatorStackEditorItemPtr& InDragItem)
{
	if (InDragItem->IsA<UActorModifierCoreBase>() && InDragItem->GetValueCount() == 1)
	{
		const bool bIsStack = InDragItem->IsA<UActorModifierCoreStack>();
		return !bIsStack;
	}

	return Super::OnIsItemDraggable(InDragItem);
}

TOptional<EItemDropZone> UActorModifierCoreEditorStackCustomization::OnItemCanAcceptDrop(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InDropZoneItem, EItemDropZone InZone)
{
	if (!InDropZoneItem->IsA<UActorModifierCoreBase>()
		|| InDropZoneItem->GetValueCount() != 1)
	{
		return Super::OnItemCanAcceptDrop(InDraggedItems, InDropZoneItem, InZone);
	}

	TSet<UActorModifierCoreBase*> DraggedModifiers;
	for (const FOperatorStackEditorItemPtr& Item : InDraggedItems)
	{
		if (!Item.IsValid()
			|| !Item->IsA<UActorModifierCoreBase>()
			|| Item->GetValueCount() != 1)
		{
			continue;
		}

		if (UActorModifierCoreBase* Modifier = Item->Get<UActorModifierCoreBase>(0))
		{
			if (!Modifier->IsModifierStack())
			{
				DraggedModifiers.Add(Modifier);
			}
		}
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	UActorModifierCoreBase* DropModifier = InDropZoneItem->Get<UActorModifierCoreBase>(0);

	if (IsValid(ModifierSubsystem) && IsValid(DropModifier->GetModifiedActor()))
	{
		TArray<UActorModifierCoreBase*> MoveModifiers;
        TArray<UActorModifierCoreBase*> CloneModifiers;
        const EActorModifierCoreStackPosition Position = InZone == EItemDropZone::AboveItem ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;

        ModifierSubsystem->GetSortedModifiers(DraggedModifiers, DropModifier->GetModifiedActor(), DropModifier, Position, MoveModifiers, CloneModifiers);

        if (!MoveModifiers.IsEmpty())
        {
        	return InZone;
        }
	}

	return Super::OnItemCanAcceptDrop(InDraggedItems, InDropZoneItem, InZone);
}

void UActorModifierCoreEditorStackCustomization::OnDropItem(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InDropZoneItem, EItemDropZone InZone)
{
	if (!InDropZoneItem->IsA<UActorModifierCoreBase>()
		|| InDropZoneItem->GetValueCount() != 1)
	{
		return;
	}

	TSet<UActorModifierCoreBase*> DraggedModifiers;
	for (const FOperatorStackEditorItemPtr& Item : InDraggedItems)
	{
		if (!Item.IsValid()
			|| !Item->IsA<UActorModifierCoreBase>()
			|| Item->GetValueCount() != 1)
		{
			continue;
		}

		if (UActorModifierCoreBase* Modifier = Item->Get<UActorModifierCoreBase>(0))
		{
			if (!Modifier->IsModifierStack())
			{
				DraggedModifiers.Add(Modifier);
			}
		}
	}

	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	UActorModifierCoreBase* DropModifier = InDropZoneItem->Get<UActorModifierCoreBase>(0);

	if (!IsValid(ModifierSubsystem) || !IsValid(DropModifier->GetModifiedActor()))
	{
		return;
	}

	TArray<UActorModifierCoreBase*> MoveModifiers;
	TArray<UActorModifierCoreBase*> CloneModifiers;
	const EActorModifierCoreStackPosition Position = InZone == EItemDropZone::AboveItem ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;

	ModifierSubsystem->GetSortedModifiers(DraggedModifiers, DropModifier->GetModifiedActor(), DropModifier, Position, MoveModifiers, CloneModifiers);

	if (MoveModifiers.IsEmpty())
	{
		return;
	}

	FText FailReason;
	FActorModifierCoreStackMoveOp MoveOp;
	MoveOp.bShouldTransact = true;
	MoveOp.FailReason = &FailReason;
	MoveOp.MovePosition = Position;
	MoveOp.MovePositionContext = DropModifier;

	ModifierSubsystem->MoveModifiers(MoveModifiers, DropModifier->GetModifierStack(), MoveOp);

	if (!FailReason.IsEmpty())
	{
		FNotificationInfo NotificationInfo(FailReason);
		NotificationInfo.ExpireDuration = 3.f;
		NotificationInfo.bFireAndForget = true;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	Super::OnDropItem(InDraggedItems, InDropZoneItem, InZone);
}

bool UActorModifierCoreEditorStackCustomization::ShouldFocusCustomization(const FOperatorStackEditorContext& InContext) const
{
	if (InContext.GetItems().IsEmpty())
	{
		return false;
	}
	
	return InContext.GetItems().Last().IsValid()
		&& (InContext.GetItems().Last()->IsA<UActorModifierCoreBase>() || InContext.GetItems().Last()->IsA<UActorModifierCoreComponent>());
}

void UActorModifierCoreEditorStackCustomization::FillStackHeaderMenu(UToolMenu* InToolMenu) const
{
	if (!InToolMenu)
	{
		return;
	}

	UOperatorStackEditorMenuContext* const AddModifierContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!AddModifierContext)
	{
		return;
	}

	FOperatorStackEditorContextPtr Context = AddModifierContext->GetContext();
	if (!Context)
	{
		return;
	}

	const UActorModifierCoreEditorSubsystem* ModifierExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();
	if (!IsValid(ModifierExtensionSubsystem))
	{
		return;
	}

	TSet<TWeakObjectPtr<UObject>> ContextObjects;
	for (const FOperatorStackEditorItemPtr& ContextItem : Context->GetItems())
	{
		if (ContextItem->IsA<UObject>())
		{
			for (UObject* ContextObject : ContextItem->GetAsArray<UObject>())
			{
				ContextObjects.Add(ContextObject);
			}
		}
	}

	const FActorModifierCoreEditorMenuContext MenuContext(ContextObjects);
	FActorModifierCoreEditorMenuOptions MenuOptions(EActorModifierCoreEditorMenuType::Add);
	MenuOptions.CreateSubMenu(false);
	ModifierExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, MenuOptions);
}

void UActorModifierCoreEditorStackCustomization::FillItemHeaderActionMenu(UToolMenu* InToolMenu) const
{
	using namespace UE::ActorModifierCoreEditor;

	if (!InToolMenu)
	{
		return;
	}

	const UOperatorStackEditorMenuContext* const MenuContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	const FOperatorStackEditorItemPtr ItemContext = MenuContext->GetItem();
	if (!ItemContext)
	{
		return;
	}

	// Add freeze & profiling stat toggle entry
	if (ItemContext->IsA<UActorModifierCoreStack>())
	{
		const FToolMenuEntry FreezeModifierStackAction = FToolMenuEntry::InitToolBarButton(
			TEXT("FreezeModifierStackMenuEntry")
			, FUIAction(
				FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::ToggleModifierStackFrozenAction, ItemContext)
				, FCanExecuteAction()
				, FIsActionChecked::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::IsModifierStackFrozen, ItemContext))
			, FText::GetEmpty()
			, Private::GetStackPropertyTooltip(GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierStackFrozen))
			, FSlateIcon(FActorModifierCoreEditorStyle::Get().GetStyleSetName(), "FreezeStack")
			, EUserInterfaceActionType::ToggleButton
		);
		InToolMenu->AddMenuEntry(FreezeModifierStackAction.Name, FreezeModifierStackAction);

		const FToolMenuEntry EnableProfilingModifierAction = FToolMenuEntry::InitToolBarButton(
			TEXT("EnableProfilingModifierMenuEntry")
			, FUIAction(
				FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::ToggleModifierProfilingAction, ItemContext)
				, FCanExecuteAction()
				, FIsActionChecked::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::IsModifierProfiling, ItemContext))
			, FText::GetEmpty()
			, FText::GetEmpty()
			, FSlateIcon(FActorModifierCoreEditorStyle::Get().GetStyleSetName(), "Profiling")
			, EUserInterfaceActionType::ToggleButton
		);

		InToolMenu->AddMenuEntry(EnableProfilingModifierAction.Name, EnableProfilingModifierAction);
	}

	// Add remove modifier entry
	const FToolMenuEntry RemoveModifierAction = FToolMenuEntry::InitToolBarButton(
		TEXT("RemoveModifierMenuEntry")
		, FUIAction(
			FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::RemoveModifierAction, ItemContext)
			, FCanExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::CanRemoveModifier, ItemContext)
		  )
		, FText::GetEmpty()
		, FText::GetEmpty()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete")
	);

	InToolMenu->AddMenuEntry(RemoveModifierAction.Name, RemoveModifierAction);
}

void UActorModifierCoreEditorStackCustomization::FillItemContextActionMenu(UToolMenu* InToolMenu) const
{
	using namespace UE::ActorModifierCoreEditor;

	if (!InToolMenu)
	{
		return;
	}

	const UOperatorStackEditorMenuContext* const MenuContext = InToolMenu->FindContext<UOperatorStackEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	const FOperatorStackEditorItemPtr ItemContext = MenuContext->GetItem();
	if (!ItemContext)
	{
		return;
	}

	if (ItemContext->IsA<UActorModifierCoreStack>())
	{
		const FToolMenuEntry FreezeModifierStackAction = FToolMenuEntry::InitMenuEntry(
			TEXT("FreezeModifierStackMenuEntry")
			, LOCTEXT("FreezeModifierStack", "Freeze Modifier Stack")
			, Private::GetStackPropertyTooltip(GET_MEMBER_NAME_CHECKED(UActorModifierCoreStack, bModifierStackFrozen))
			, FSlateIcon(FActorModifierCoreEditorStyle::Get().GetStyleSetName(), "FreezeStack")
			, FUIAction(
				FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::ToggleModifierStackFrozenAction, ItemContext)
				, FCanExecuteAction()
				, FIsActionChecked::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::IsModifierStackFrozen, ItemContext))
			, EUserInterfaceActionType::ToggleButton
		);

		InToolMenu->AddMenuEntry(FreezeModifierStackAction.Name, FreezeModifierStackAction);

		const FToolMenuEntry EnableProfilingModifierAction = FToolMenuEntry::InitMenuEntry(
			TEXT("EnableProfilingModifierMenuEntry")
			, LOCTEXT("EnableProfilingModifier", "Toggle profiling")
			, FText::GetEmpty()
			, FSlateIcon(FActorModifierCoreEditorStyle::Get().GetStyleSetName(), "Profiling")
			, FUIAction(
				FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::ToggleModifierProfilingAction, ItemContext)
				, FCanExecuteAction()
				, FIsActionChecked::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::IsModifierProfiling, ItemContext))
			, EUserInterfaceActionType::ToggleButton);

		InToolMenu->AddMenuEntry(EnableProfilingModifierAction.Name, EnableProfilingModifierAction);
	}

	// Link menu entry to commands list for delete
	{
		TSharedPtr<const FUICommandList> Commands;
		const TSharedPtr<FUICommandInfo> Command = FGenericCommands::Get().Delete;
		InToolMenu->Context.GetActionForCommand(Command, Commands);
		const FToolMenuEntry RemoveModifierMenuEntry = FToolMenuEntry::InitMenuEntryWithCommandList(Command, Commands);
		InToolMenu->AddMenuEntry(RemoveModifierMenuEntry.Name, RemoveModifierMenuEntry);
	}

	// Link menu entry to commands list for copy
	{
		TSharedPtr<const FUICommandList> Commands;
		const TSharedPtr<FUICommandInfo> Command = FGenericCommands::Get().Copy;
		InToolMenu->Context.GetActionForCommand(Command, Commands);
		const FToolMenuEntry CopyModifierMenuEntry = FToolMenuEntry::InitMenuEntryWithCommandList(Command, Commands);
		InToolMenu->AddMenuEntry(CopyModifierMenuEntry.Name, CopyModifierMenuEntry);
	}

	// Link menu entry to commands list for paste
	{
		TSharedPtr<const FUICommandList> Commands;
		const TSharedPtr<FUICommandInfo> Command = FGenericCommands::Get().Paste;
		InToolMenu->Context.GetActionForCommand(Command, Commands);
		const FToolMenuEntry PasteModifierMenuEntry = FToolMenuEntry::InitMenuEntryWithCommandList(Command, Commands);
		InToolMenu->AddMenuEntry(PasteModifierMenuEntry.Name, PasteModifierMenuEntry);
	}
}

bool UActorModifierCoreEditorStackCustomization::CanRemoveModifier(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return false;
	}

	for (UActorModifierCoreBase* Modifier : InItem->GetAsArray<UActorModifierCoreBase>())
	{
		if (!Modifier)
		{
			continue;
		}

		const UActorModifierCoreStack* const ModifierStack = Modifier->GetModifierStack();
		if (ModifierStack && ModifierStack->IsModifierStackFrozen())
		{
			return false;
		}
	}

	return true;
}

void UActorModifierCoreEditorStackCustomization::RemoveModifierAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!CanRemoveModifier(InItem))
	{
		return;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return;
	}

	if (InItem->IsA<UActorModifierCoreStack>())
	{
		const TSet<UActorModifierCoreStack*> ModifierStacks(InItem->GetAsArray<UActorModifierCoreStack>());

		if (!ModifierSubsystem->RemoveModifierStacks(ModifierStacks, /** Transact */true))
		{
			UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Could not remove modifier stacks from actors")
		}
	}
	else
	{
		TSet<UActorModifierCoreBase*> Modifiers;
		Modifiers.Append(InItem->GetAsArray<UActorModifierCoreBase>());

		FText OutFailReason;
		FActorModifierCoreStackRemoveOp RemoveOp;
		RemoveOp.bShouldTransact = true;
		RemoveOp.FailReason = &OutFailReason;

		if (!ModifierSubsystem->RemoveModifiers(Modifiers, RemoveOp))
		{
			FNotificationInfo NotificationInfo(OutFailReason);
			NotificationInfo.ExpireDuration = 3.0f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

bool UActorModifierCoreEditorStackCustomization::CanCopyModifier(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue() || InItem->GetValueCount() != 1)
	{
		return false;
	}

	return true;
}

void UActorModifierCoreEditorStackCustomization::CopyModifierAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!CanCopyModifier(InItem))
	{
		return;
	}

	// Should only contain one modifier since we only allow action for one
	TArray<TSharedPtr<FJsonValue>> JsonModifiers;
	for (UActorModifierCoreBase* Modifier : InItem->GetAsArray<UActorModifierCoreBase>())
	{
		TMap<FName, FString> ModifierPropertiesHandlesMap;
		if (!CreatePropertiesHandlesMapFromModifier(Modifier, ModifierPropertiesHandlesMap))
		{
			continue;
		}

		FActorModifierCoreEditorPropertiesWrapper ModifierPropertiesWrapper;
		ModifierPropertiesWrapper.ModifierName = Modifier->GetModifierName();
		ModifierPropertiesWrapper.PropertiesHandlesAsStringMap = ModifierPropertiesHandlesMap;

		TSharedRef<FJsonObject> PropertiesJsonObject = MakeShared<FJsonObject>();
		FJsonObjectConverter::UStructToJsonObject(FActorModifierCoreEditorPropertiesWrapper::StaticStruct(), &ModifierPropertiesWrapper, PropertiesJsonObject, 0 /* CheckFlags */, 0 /* SkipFlags */);
		JsonModifiers.Add(MakeShared<FJsonValueObject>(PropertiesJsonObject));
	}

	FString SerializedString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&SerializedString);
	if (!FJsonSerializer::Serialize(JsonModifiers, Writer))
	{
		UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Unable to serialize the selected modifier(s) into Json format");
		return;
	}

	// Add prefix to quickly identify whether current clipboard is from modifiers or not
	SerializedString = *FString::Printf(TEXT("%s%s")
		, *PropertiesWrapperPrefix
		, *SerializedString);

	FPlatformApplicationMisc::ClipboardCopy(*SerializedString);
}

bool UActorModifierCoreEditorStackCustomization::CanPasteModifier(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue() || InItem->GetValueCount() != 1)
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return ClipboardContent.StartsWith(PropertiesWrapperPrefix);
}

void UActorModifierCoreEditorStackCustomization::PasteModifierAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!CanPasteModifier(InItem))
	{
		return;
	}

	// We only allow usage of this action when one item is selected but we could support multi selection later
	TArray<FActorModifierCoreEditorPropertiesWrapper> ModifierPropertiesWrapper;
	if (!GetModifierPropertiesWrapperFromClipboard(ModifierPropertiesWrapper))
	{
		return;
	}

	const FName ModifierName = ModifierPropertiesWrapper[0].ModifierName;
	TArray<UActorModifierCoreBase*> TargetModifiers;

	if (InItem->IsA<UActorModifierCoreStack>())
	{
		TSet<AActor*> Actors;
		for (const UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
		{
			Actors.Add(ModifierStack->GetModifiedActor());
		}

		if (!AddModifierFromClipboard(Actors, ModifierName, TargetModifiers))
		{
			return;
		}
	}
	else
	{
		TargetModifiers = InItem->GetAsArray<UActorModifierCoreBase>();
	}

	if (TargetModifiers.Num() != ModifierPropertiesWrapper.Num())
	{
		UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Unable to set properties from %ls modifier due to target modifiers (%i) and modifiers properties (%i) count mismatch", *ModifierName.ToString(), TargetModifiers.Num(), ModifierPropertiesWrapper.Num());
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("PasteModifierProperties", "Paste Modifier Properties"));

	for (int32 Index = 0; Index < TargetModifiers.Num(); Index++)
	{
		UActorModifierCoreBase* TargetModifier = TargetModifiers[Index];

		if (!IsValid(TargetModifier))
		{
			continue;
		}

		const FActorModifierCoreEditorPropertiesWrapper ModifierProperties = ModifierPropertiesWrapper[Index];

		if (ModifierName == TargetModifier->GetModifierName())
		{
			TargetModifier->Modify();
			UpdateModifierFromPropertiesHandlesMap(TargetModifier, ModifierProperties.PropertiesHandlesAsStringMap);
		}
		else
		{
			const FString SourceModifierName = ModifierProperties.ModifierName.ToString();
			const FString DestinationModifierName = TargetModifier->GetModifierName().ToString();

			UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Unable to copy properties from %ls modifier to %ls modifier", *SourceModifierName, *DestinationModifierName);
		}
	}
}

bool UActorModifierCoreEditorStackCustomization::IsModifierStackFrozen(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return false;
	}

	for (const UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
	{
		if (ModifierStack && ModifierStack->IsModifierStackFrozen())
		{
			return true;
		}
	}

	return false;
}

void UActorModifierCoreEditorStackCustomization::ToggleModifierStackFrozenAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return;
	}

	const bool bIsModifierFrozen = IsModifierStackFrozen(InItem);
	for (UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
	{
		if (ModifierStack)
		{
			ModifierStack->SetModifierStackFrozen(!bIsModifierFrozen);	
		}
	}
}

bool UActorModifierCoreEditorStackCustomization::IsModifierProfiling(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return false;
	}

	for (const UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
	{
		if (!ModifierStack->IsModifierProfiling())
		{
			return false;
		}
	}

	return true;
}

void UActorModifierCoreEditorStackCustomization::ToggleModifierProfilingAction(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid() || !InItem->HasValue())
	{
		return;
	}

	for (UActorModifierCoreStack* ModifierStack : InItem->GetAsArray<UActorModifierCoreStack>())
	{
		ModifierStack->SetModifierProfiling(!ModifierStack->IsModifierProfiling());
	}
}

void UActorModifierCoreEditorStackCustomization::OnModifierAdded(UActorModifierCoreBase* InModifier, EActorModifierCoreEnableReason InReason)
{
	if (InReason == EActorModifierCoreEnableReason::User)
	{
		OnModifierUpdated(InModifier);
	}
}

void UActorModifierCoreEditorStackCustomization::OnModifierRemoved(UActorModifierCoreBase* InModifier, EActorModifierCoreDisableReason InReason)
{
	if (InModifier)
	{
		RefreshActiveSelection(InModifier->GetRootModifierStack(), /** Force */true);
	}
}

void UActorModifierCoreEditorStackCustomization::OnModifierUpdated(UActorModifierCoreBase* InModifier)
{
	if (InModifier)
	{
		RefreshActiveSelection(InModifier->GetRootModifierStack(), /** Force */false);
		FocusCustomization(InModifier->GetRootModifierStack());
	}
}

TSharedRef<FUICommandList> UActorModifierCoreEditorStackCustomization::CreateModifierCommands(FOperatorStackEditorItemPtr InItem)
{
	TSharedRef<FUICommandList> Commands = MakeShared<FUICommandList>();

	Commands->MapAction(FGenericCommands::Get().Copy
		, FUIAction(
			FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::CopyModifierAction, InItem),
			FCanExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::CanCopyModifier, InItem)
		)
	);

	Commands->MapAction(FGenericCommands::Get().Paste
		, FUIAction(
			FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::PasteModifierAction, InItem),
			FCanExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::CanPasteModifier, InItem)
		)
	);

	Commands->MapAction(FGenericCommands::Get().Delete
		, FUIAction(
			FExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::RemoveModifierAction, InItem),
			FCanExecuteAction::CreateUObject(this, &UActorModifierCoreEditorStackCustomization::CanRemoveModifier, InItem)
		)
	);

	return Commands;
}

bool UActorModifierCoreEditorStackCustomization::CreatePropertiesHandlesMapFromModifier(UActorModifierCoreBase* InModifier, TMap<FName, FString>& OutModifierPropertiesHandlesMap) const
{
	if (!InModifier)
	{
		return false;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	// no need to keep the generator outside of this function, since we are converting handles to string
	const FPropertyRowGeneratorArgs RowGeneratorArgs;
	const TSharedRef<IPropertyRowGenerator> PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(RowGeneratorArgs);

	UObject* ModifierAsObj = Cast<UObject>(InModifier);
	check(ModifierAsObj);

	PropertyRowGenerator->SetObjects({ModifierAsObj});

	const TArray<TSharedRef<IDetailTreeNode>>& RootTreeNodes = PropertyRowGenerator->GetRootTreeNodes();

	for (const TSharedRef<IDetailTreeNode>& RootTreeNode : RootTreeNodes)
	{
		TArray<TSharedRef<IDetailTreeNode>> ChildrenNodes;
		RootTreeNode->GetChildren(ChildrenNodes);

		for (const TSharedRef<IDetailTreeNode>& Node : ChildrenNodes )
		{
			if (const TSharedPtr<IPropertyHandle>& PropertyHandle = Node->CreatePropertyHandle())
			{
				FName PropertyName = PropertyHandle->GetProperty()->NamePrivate;

				FString PropertyValueAsString;
				if (PropertyHandle->GetValueAsFormattedString(PropertyValueAsString, PPF_Copy) == FPropertyAccess::Success)
				{
					OutModifierPropertiesHandlesMap.Add(PropertyName, PropertyValueAsString);
				}
			}
		}
	}

	return true;
}

bool UActorModifierCoreEditorStackCustomization::GetModifierPropertiesWrapperFromClipboard(TArray<FActorModifierCoreEditorPropertiesWrapper>& OutModifierProperties) const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	if (!ClipboardContent.StartsWith(PropertiesWrapperPrefix))
	{
		return false;
	}

	// remove prefix, this is not part of the modifier properties json data
	ClipboardContent.RightChopInline(PropertiesWrapperPrefix.Len());

	TArray<TSharedPtr<FJsonValue>> JsonModifiers;
	const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ClipboardContent);

	if (!FJsonSerializer::Deserialize(Reader, JsonModifiers))
	{
		UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Unable to deserialize the clipboard text into Json format");
		return false;
	}

	if (JsonModifiers.IsEmpty())
	{
		UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "No Json modifiers data available to paste");
		return false;
	}

	if (!FJsonObjectConverter::JsonArrayToUStruct(JsonModifiers, &OutModifierProperties))
	{
		UE_LOGF(LogActorModifierCoreEditorStackCustomization, Warning, "Invalid Json modifiers properties found");
		return false;
	}

	return !OutModifierProperties.IsEmpty();
}

bool UActorModifierCoreEditorStackCustomization::UpdateModifierFromPropertiesHandlesMap(UActorModifierCoreBase* InModifier, const TMap<FName, FString>& InModifierPropertiesHandlesMap) const
{
	if (!InModifier)
	{
		return false;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	// no need to keep the generator outside of this function, since we are converting handles to string
	const FPropertyRowGeneratorArgs RowGeneratorArgs;
	const TSharedRef<IPropertyRowGenerator> PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(RowGeneratorArgs);

	UObject* ModifierAsObj = Cast<UObject>(InModifier);
	check(ModifierAsObj);

	PropertyRowGenerator->SetObjects({ModifierAsObj});

	const TArray<TSharedRef<IDetailTreeNode>>& RootTreeNodes = PropertyRowGenerator->GetRootTreeNodes();

	for (const TSharedRef<IDetailTreeNode>& RootTreeNode : RootTreeNodes)
	{
		TArray<TSharedRef<IDetailTreeNode>> ChildrenNodes;
		RootTreeNode->GetChildren(ChildrenNodes);

		for (const TSharedRef<IDetailTreeNode>& Node : ChildrenNodes )
		{
			if (const TSharedPtr<IPropertyHandle>& PropertyHandle = Node->CreatePropertyHandle())
			{
				FName PropertyName = PropertyHandle->GetProperty()->NamePrivate;

				if (InModifierPropertiesHandlesMap.Contains(PropertyName))
				{
					FString PropertyValueAsString = InModifierPropertiesHandlesMap[PropertyName];
					PropertyHandle->SetValueFromFormattedString(PropertyValueAsString, EPropertyValueSetFlags::InstanceObjects);
				}
			}
		}
	}

	return true;
}

bool UActorModifierCoreEditorStackCustomization::AddModifierFromClipboard(const TSet<AActor*>& InActors, FName InModifierName, TArray<UActorModifierCoreBase*>& OutNewModifiers) const
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	if (!IsValid(ModifierSubsystem))
	{
		return false;
	}

	FText OutFailReason;
	FActorModifierCoreStackInsertOp AddOp;
	AddOp.bShouldTransact = true;
	AddOp.FailReason = &OutFailReason;
	AddOp.NewModifierName = InModifierName;

	OutNewModifiers = ModifierSubsystem->AddActorsModifiers(InActors, AddOp);

	if (!OutFailReason.IsEmpty())
	{
		FNotificationInfo NotificationInfo(OutFailReason);
		NotificationInfo.ExpireDuration = 3.0f;
		NotificationInfo.bFireAndForget = true;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		return false;
	}

	return !OutNewModifiers.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
