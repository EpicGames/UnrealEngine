// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "ActorModifierCoreEditorStackCustomization.generated.h"

class UActorModifierCoreStack;
class UActorModifierCoreBase;
class UToolMenu;
enum class EActorModifierCoreDisableReason : uint8;
enum class EActorModifierCoreEnableReason : uint8;
enum class EItemDropZone;

/** Struct used for copy pasting modifiers in clipboard */
USTRUCT()
struct FActorModifierCoreEditorPropertiesWrapper
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName ModifierName;

	UPROPERTY()
	TMap<FName, FString> PropertiesHandlesAsStringMap;
};

/** Modifier customization for stack tab */
UCLASS()
class UActorModifierCoreEditorStackCustomization : public UOperatorStackEditorStackCustomization
{
	GENERATED_BODY()

public:
	static inline const FString PropertiesWrapperPrefix = TEXT("ActorModifierCoreEditorPropertiesWrapper");

	UActorModifierCoreEditorStackCustomization();
	virtual ~UActorModifierCoreEditorStackCustomization() override;

	//~ Begin UOperatorStackEditorStackCustomization
	virtual bool GetRootItem(const FOperatorStackEditorContext& InContext, FOperatorStackEditorItemPtr& OutRootItem) const override;
	virtual bool GetChildrenItem(const FOperatorStackEditorItemPtr& InItem, TArray<FOperatorStackEditorItemPtr>& OutChildrenItems) const override;
	virtual void CustomizeStackHeader(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder) override;
	virtual void CustomizeItemHeader(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder) override;
	virtual void CustomizeItemBody(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InBodyBuilder) override;
	virtual void CustomizeItemFooter(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorFooterBuilder& InFooterBuilder) override;
	virtual bool OnIsItemDraggable(const FOperatorStackEditorItemPtr& InDragItem) override;
	virtual TOptional<EItemDropZone> OnItemCanAcceptDrop(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InDropZoneItem, EItemDropZone InZone) override;
	virtual void OnDropItem(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InDropZoneItem, EItemDropZone InZone) override;
	virtual bool ShouldFocusCustomization(const FOperatorStackEditorContext& InContext) const override;
	//~ UOperatorStackEditorStackCustomization

protected:
	/** Populates the header menu of the whole customization stack */
	void FillStackHeaderMenu(UToolMenu* InToolMenu) const;

	/** Populates the header action menu for items */
	void FillItemHeaderActionMenu(UToolMenu* InToolMenu) const;

	/** Populates the context action menu for item */
	void FillItemContextActionMenu(UToolMenu* InToolMenu) const;

	/** Remove modifier action */
	bool CanRemoveModifier(FOperatorStackEditorItemPtr InItem) const;
	void RemoveModifierAction(FOperatorStackEditorItemPtr InItem) const;

	/** Copy modifier action */
	bool CanCopyModifier(FOperatorStackEditorItemPtr InItem) const;
	void CopyModifierAction(FOperatorStackEditorItemPtr InItem) const;

	/** Paste modifier action */
	bool CanPasteModifier(FOperatorStackEditorItemPtr InItem) const;
	void PasteModifierAction(FOperatorStackEditorItemPtr InItem) const;

	/** Toggle the frozen state for a whole stack */
	bool IsModifierStackFrozen(FOperatorStackEditorItemPtr InItem) const;
	void ToggleModifierStackFrozenAction(FOperatorStackEditorItemPtr InItem) const;

	/** Toggle the profiling for a whole stack */
	bool IsModifierProfiling(FOperatorStackEditorItemPtr InItem) const;
	void ToggleModifierProfilingAction(FOperatorStackEditorItemPtr InItem) const;

	void OnModifierAdded(UActorModifierCoreBase* InModifier, EActorModifierCoreEnableReason InReason);
	void OnModifierRemoved(UActorModifierCoreBase* InModifier, EActorModifierCoreDisableReason InReason);
	void OnModifierUpdated(UActorModifierCoreBase* InModifier);

	TSharedRef<FUICommandList> CreateModifierCommands(FOperatorStackEditorItemPtr InItem);

	bool CreatePropertiesHandlesMapFromModifier(UActorModifierCoreBase* InModifier, TMap<FName, FString>& OutModifierPropertiesHandlesMap) const;
	bool GetModifierPropertiesWrapperFromClipboard(TArray<FActorModifierCoreEditorPropertiesWrapper>& OutModifierProperties) const;
	bool UpdateModifierFromPropertiesHandlesMap(UActorModifierCoreBase* InModifier, const TMap<FName, FString>& InModifierPropertiesHandlesMap) const;
	bool AddModifierFromClipboard(const TSet<AActor*>& InActors, FName InModifierName, TArray<UActorModifierCoreBase*>& OutNewModifiers) const;
};