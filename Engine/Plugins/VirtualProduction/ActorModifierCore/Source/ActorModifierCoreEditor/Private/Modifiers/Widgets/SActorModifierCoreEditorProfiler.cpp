// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Widgets/SActorModifierCoreEditorProfiler.h"

#include "Framework/Application/IMenu.h"
#include "Framework/Application/SlateApplication.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "Widgets/SBoxPanel.h"

void SActorModifierCoreEditorProfiler::Construct(const FArguments& InArgs, TSharedPtr<FActorModifierCoreProfiler> InProfiler)
{
	check(InProfiler.IsValid())
	
	ProfilerWeak = InProfiler;

	UActorModifierCoreEditorSubsystem* ExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!ExtensionSubsystem->ModifierProfilerStats.Contains(InProfiler->GetProfilerType()))
	{
		ExtensionSubsystem->ModifierProfilerStats.Add(InProfiler->GetProfilerType(), InProfiler->GetMainProfilingStats());
	}

	SetupProfilerStats(InProfiler);

	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SActorModifierCoreEditorProfiler::GetProfilerVisibility));

	const TSharedRef<SUniformWrapPanel> GridPanel = SNew(SUniformWrapPanel)
		.HAlign(HAlign_Fill)
		.EvenRowDistribution(true)
		.NumColumnsOverride(3)
		.SlotPadding(Padding);

	for (const TPair<FName, FActorModifierCoreEditorProfilerStat>& Stat : ProfilerStats)
	{
		GridPanel->AddSlot()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SActorModifierCoreEditorProfiler::GetStatVisibility, Stat.Key)
			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.FillWidth(0.7f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(FText::FromString(FName::NameToDisplayString(Stat.Key.ToString(), false)))
			]
			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.FillWidth(0.3f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Right)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.ColorAndOpacity(Stat.Value.ValueColor)
				.Text(Stat.Value.ValueText)
			]
		];
	}
	
	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	.Padding(Padding)
	[
		GridPanel
	];
}

void SActorModifierCoreEditorProfiler::SetupProfilerStats(TSharedPtr<FActorModifierCoreProfiler> InProfiler)
{
	ScanPropertiesStats();
}

void SActorModifierCoreEditorProfiler::ScanPropertiesStats()
{
	const TSharedPtr<FActorModifierCoreProfiler> Profiler = ProfilerWeak.Pin();
	if (!Profiler.IsValid())
	{
		return;
	}

	const TSharedPtr<FStructOnScope> Struct = Profiler->GetStructProfilerStats();
	if (!Struct.IsValid())
	{
		return;
	}

	TFunction<void(const UStruct*, uint8*)> RecursivePropertyLookup = [this, &RecursivePropertyLookup](const UStruct* InStruct, uint8* InStructPtr)
	{
		for (FProperty* Property : TFieldRange<FProperty>(InStruct))
		{
			if (!Property)
			{
				continue;
			}
			
			// Only handle primitive types or struct/object
			if (Property->IsA<FSetProperty>()
				|| Property->IsA<FArrayProperty>()
				|| Property->IsA<FMapProperty>())
			{
				continue;
			}

			uint8* ValuePtr = Property->ContainerPtrToValuePtr<uint8>(InStructPtr);
			
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				RecursivePropertyLookup(StructProperty->Struct, ValuePtr);
			}
			else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				RecursivePropertyLookup(ObjectProperty->PropertyClass, ValuePtr);
			}
			else
			{
				HandlePropertyStat(Property, ValuePtr);
			}
		}
	};

	const UStruct* StructType = Struct->GetStruct();
	uint8* StructPtr = Struct->GetStructMemory();
	RecursivePropertyLookup(StructType, StructPtr);
}

void SActorModifierCoreEditorProfiler::HandlePropertyStat(FProperty* InProperty, uint8* InValuePtr)
{
	const FName PropertyName = InProperty->GetFName();

	// We only care about float or int at this point, extend if needed
	if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
	{
		FActorModifierCoreEditorProfilerStat* Stat = AddProfilerStat(PropertyName);
		
		if (NumericProperty->IsA<FDoubleProperty>()
			|| NumericProperty->IsA<FFloatProperty>())
		{
			Stat->ValueText = TAttribute<FText>::CreateLambda([NumericProperty, InValuePtr, Suffix = Stat->Suffix]()
			{
				const double OutValue = NumericProperty->GetFloatingPointPropertyValue(InValuePtr);

				return FText::FromString(FString::Printf(TEXT("%.3f %s"), OutValue, *Suffix));
			});
		}
		else
		{
			Stat->ValueText = TAttribute<FText>::CreateLambda([NumericProperty, InValuePtr, Suffix = Stat->Suffix]()
			{
				const int64 OutValue = NumericProperty->GetSignedIntPropertyValue(InValuePtr);

				return FText::FromString(FString::Printf(TEXT("%lld %s"), OutValue, *Suffix));
			});
		}
	}
}

FReply SActorModifierCoreEditorProfiler::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		ShowContextMenu(MouseEvent.GetLastScreenSpacePosition());
		return FReply::Handled();
	}
	
	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SActorModifierCoreEditorProfiler::ShowContextMenu(const FVector2D& InPosition)
{
	if (ContextMenu.IsValid())
	{
		ContextMenu->Dismiss();
	}
	
	ContextMenu = FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			OnGetContextMenuWidget(),
			InPosition,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}

FActorModifierCoreEditorProfilerStat* SActorModifierCoreEditorProfiler::AddProfilerStat(const FName& InName)
{
	if (InName.IsNone())
	{
		return nullptr;
	}
	
	FActorModifierCoreEditorProfilerStat NewStat;
	NewStat.Name = InName;
	NewStat.ValueText.Set(FText::GetEmpty());
	NewStat.ValueColor.Set(FLinearColor::White);

	OnProfilerStatAdded(NewStat);
	
	return &ProfilerStats.Emplace(NewStat.Name, NewStat);
}

FActorModifierCoreEditorProfilerStat* SActorModifierCoreEditorProfiler::GetProfilerStat(const FName& InName)
{
	return ProfilerStats.Find(InName);
}

void SActorModifierCoreEditorProfiler::OnProfilerStatAdded(FActorModifierCoreEditorProfilerStat& InStat)
{
	TSharedPtr<FActorModifierCoreProfiler> Profiler = ProfilerWeak.Pin();

	// Add unit suffix to time stats
	if (InStat.Name == FActorModifierCoreProfiler::ExecutionTimeName
		|| InStat.Name == FActorModifierCoreProfiler::AverageExecutionTimeName
		|| InStat.Name == FActorModifierCoreProfiler::TotalExecutionTimeName)
	{
		InStat.Suffix = TEXT("ms");
	}

	// Add color info to execution stat
	if (InStat.Name == FActorModifierCoreProfiler::ExecutionTimeName
		&& !Profiler->GetModifier()->IsModifierStack())
	{
		InStat.ValueColor = TAttribute<FSlateColor>::CreateLambda([Profiler]()
		{
			const TSharedPtr<FActorModifierCoreProfiler> StackProfiler = Profiler->GetModifier()->GetRootModifierStack()->GetProfiler();

			const double StackExecutionTime = StackProfiler->GetExecutionTime();
			const double ModifierExecutionTime = Profiler->GetExecutionTime();

			if (StackExecutionTime == 0.f
				|| ModifierExecutionTime == 0.f)
			{
				return FLinearColor::White;
			}
			
			return FMath::Lerp<FLinearColor>(FLinearColor::Green, FLinearColor::Red, ModifierExecutionTime / StackExecutionTime);
		});
	}
}

void SActorModifierCoreEditorProfiler::TogglePinProfilerStat(FName InName)
{
	SetPinProfilerStat(InName, !IsProfilerStatPinned(InName));
}

void SActorModifierCoreEditorProfiler::SetPinProfilerStat(const FName& InName, bool bInIsPinned)
{
	const TSharedPtr<FActorModifierCoreProfiler> Profiler = ProfilerWeak.Pin();
	UActorModifierCoreEditorSubsystem* ExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!Profiler.IsValid() || !ExtensionSubsystem)
	{
		return;
	}

	TSet<FName>* PinnedStats = ExtensionSubsystem->ModifierProfilerStats.Find(Profiler->GetProfilerType());
	if (!PinnedStats)
	{
		return;
	}

	if (bInIsPinned)
	{
		PinnedStats->Add(InName);
	}
	else
	{
		PinnedStats->Remove(InName);
	}
}

TSharedRef<SWidget> SActorModifierCoreEditorProfiler::OnGetContextMenuWidget()
{
	FMenuBuilder ContextMenuBuilder(true, nullptr);
	
	for (const TPair<FName, FActorModifierCoreEditorProfilerStat>& Stat : ProfilerStats)
	{
		const FName StatName = Stat.Key;
		
		ContextMenuBuilder.AddMenuEntry(
			FText::FromName(StatName),
			FText::FromName(StatName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SActorModifierCoreEditorProfiler::TogglePinProfilerStat, StatName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SActorModifierCoreEditorProfiler::IsProfilerStatPinned, StatName)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	return ContextMenuBuilder.MakeWidget();
}

EVisibility SActorModifierCoreEditorProfiler::GetStatVisibility(FName InName) const
{
	const TSharedPtr<FActorModifierCoreProfiler> Profiler = ProfilerWeak.Pin();
	const UActorModifierCoreEditorSubsystem* ExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!Profiler.IsValid() || !ExtensionSubsystem)
	{
		return EVisibility::Collapsed;
	}

	const TSet<FName>* PinnedStats = ExtensionSubsystem->ModifierProfilerStats.Find(Profiler->GetProfilerType());
	if (!PinnedStats)
	{
		return EVisibility::Collapsed;
	}

	return PinnedStats->Contains(InName) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SActorModifierCoreEditorProfiler::GetProfilerVisibility() const
{
	const TSharedPtr<FActorModifierCoreProfiler> Profiler = ProfilerWeak.Pin();
	
	if (!Profiler.IsValid())
	{
		return EVisibility::Collapsed;
	}

	const UActorModifierCoreBase* Modifier = Profiler->GetModifier();
	
	if (!Modifier)
	{
		return EVisibility::Collapsed;
	}
	
	return Modifier->IsModifierProfiling() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SActorModifierCoreEditorProfiler::IsProfilerStatPinned(FName InName) const
{
	return GetStatVisibility(InName) == EVisibility::Visible;
}
