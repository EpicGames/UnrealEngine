// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/ActorModifierCoreEditorSubsystem.h"

#include "Editor.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Text.h"
#include "Modifiers/ActorModifierCoreEditorMenu.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Widgets/SActorModifierCoreEditorProfiler.h"
#include "ScopedTransaction.h"
#include "ActorModifierCoreEditorStyle.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "ActorModifierCoreEditorExtensionSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogActorModifierCoreEditorSubsystem, Log, All);

UActorModifierCoreEditorSubsystem::UActorModifierCoreEditorSubsystem()
	: UEditorSubsystem()
{
}

UActorModifierCoreEditorSubsystem* UActorModifierCoreEditorSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UActorModifierCoreEditorSubsystem>();
	}
	return nullptr;
}

void UActorModifierCoreEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	EngineSubsystem = UActorModifierCoreSubsystem::Get();

	check(EngineSubsystem.IsValid());

	// Initialize here if not called anywhere to setup ClassIcons
	FActorModifierCoreEditorStyle::Get();

	// Register modifier categories
	{
		const FActorModifierCoreEditorCategoryMetadata DefaultCategory
		{
			.Name = TEXT("Default"),
			.DisplayName = LOCTEXT("DefaultCategoryDisplayName", "Default"),
		};

		RegisterModifierCategory(DefaultCategory);

		const FActorModifierCoreEditorCategoryMetadata LayoutCategory
		{
			.Name = TEXT("Layout"),
			.DisplayName = LOCTEXT("LayoutCategoryDisplayName", "Layout"),
		};

		RegisterModifierCategory(LayoutCategory);

		const FActorModifierCoreEditorCategoryMetadata CameraCategory
		{
			.Name = TEXT("Camera"),
			.DisplayName = LOCTEXT("CameraCategoryDisplayName", "Camera"),
		};

		RegisterModifierCategory(CameraCategory);

		const FActorModifierCoreEditorCategoryMetadata RenderingCategory
		{
			.Name = TEXT("Rendering"),
			.DisplayName = LOCTEXT("RenderingCategoryDisplayName", "Rendering"),
		};

		RegisterModifierCategory(RenderingCategory);

		const FActorModifierCoreEditorCategoryMetadata GeometryCategory
		{
			.Name = TEXT("Geometry"),
			.DisplayName = LOCTEXT("GeometryCategoryDisplayName", "Geometry"),
		};

		RegisterModifierCategory(GeometryCategory);

		const FActorModifierCoreEditorCategoryMetadata ConversionCategory
		{
			.Name = TEXT("Conversion"),
			.DisplayName = LOCTEXT("ConversionCategoryDisplayName", "Conversion"),
		};

		RegisterModifierCategory(ConversionCategory);
	}
}

bool UActorModifierCoreEditorSubsystem::FillModifierMenu(UToolMenu* InMenu, const FActorModifierCoreEditorMenuContext& InContext, const FActorModifierCoreEditorMenuOptions& InMenuOptions) const
{
	if (!InMenu || InContext.IsEmpty())
	{
		return false;
	}

	const FActorModifierCoreEditorMenuData MenuData(InContext, InMenuOptions);
	FToolMenuSection* ContextModifiersSection = nullptr;
	if (InMenuOptions.ShouldCreateSubMenu())
	{
		static const FName ModifierSection("ContextModifierActions");

		ContextModifiersSection = InMenu->FindSection(ModifierSection);
		if (!ContextModifiersSection)
		{
			ContextModifiersSection = &InMenu->AddSection(ModifierSection
				, LOCTEXT("ContextModifierActions", "Modifiers Actions")
				, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
		}
	}

	if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::Add)
	{
		if (InContext.ContainsAnyActor())
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextAddModifiersActions"),
					LOCTEXT("AddModifiers.Label", "Add Modifiers"),
					LOCTEXT("AddModifiers.Tooltip", "Add modifiers to this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendAddModifierMenu, MenuData));
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendAddModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::Delete)
	{
		if (InContext.ContainsNonEmptyStack() || InContext.ContainsAnyModifier())
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextRemoveModifierActions"),
					LOCTEXT("RemoveModifier.Label", "Remove Modifiers"),
					LOCTEXT("EnableModifier.Tooltip", "Remove modifiers from this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendRemoveModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendRemoveModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::Move)
	{
		if (InContext.ContainsOnlyModifier() && InContext.GetContextModifiers().Num() == 1)
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextMoveModifiersActions"),
					LOCTEXT("MoveModifiers.Label", "Move Modifiers"),
					LOCTEXT("MovModifiers.Tooltip", "Move modifiers in the stack"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendMoveModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendMoveModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::Enable)
	{
		if (InContext.ContainsDisabledModifier() || InContext.ContainsDisabledStack())
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextEnableModifiersAction"),
					LOCTEXT("EnableModifiers.Label", "Enable Modifier"),
					LOCTEXT("EnableModifiers.Tooltip", "Enable modifier from this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendEnableModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendEnableModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::Disable)
	{
		if (InContext.ContainsEnabledModifier() || InContext.ContainsEnabledStack())
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextDisableModifiersAction"),
					LOCTEXT("DisableModifiers.Label", "Disable Modifier"),
					LOCTEXT("DisableModifiers.Tooltip", "Disable modifier from this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendEnableModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendEnableModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::InsertBefore)
	{
		if (InContext.ContainsOnlyModifier() && InContext.GetContextModifiers().Num() == 1)
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextInsertBeforeModifiersActions"),
					LOCTEXT("InsertBeforeModifiers.Label", "Insert Modifiers Before"),
					LOCTEXT("InsertBeforeModifiers.Tooltip", "Insert modifiers before this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendInsertModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendInsertModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}
	else if (InMenuOptions.GetMenuType() == EActorModifierCoreEditorMenuType::InsertAfter)
	{
		if (InContext.ContainsOnlyModifier() && InContext.GetContextModifiers().Num() == 1)
		{
			if (InMenuOptions.ShouldCreateSubMenu())
			{
				ContextModifiersSection->AddSubMenu(
					TEXT("ContextInsertAfterModifiersActions"),
					LOCTEXT("InsertAfterModifiers.Label", "Insert Modifiers After"),
					LOCTEXT("InsertAfterModifiers.Tooltip", "Insert modifiers after this selection"),
					FNewToolMenuDelegate::CreateLambda(&UE::ActorModifierCoreEditor::OnExtendInsertModifierMenu, MenuData)
				);
			}
			else
			{
				UE::ActorModifierCoreEditor::OnExtendInsertModifierMenu(InMenu, MenuData);
			}
			return true;
		}
	}

	return false;
}

TSharedPtr<SActorModifierCoreEditorProfiler> UActorModifierCoreEditorSubsystem::CreateProfilerWidget(TSharedPtr<FActorModifierCoreProfiler> InProfiler)
{
	if (!InProfiler.IsValid())
	{
		return nullptr;
	}

	if (const TFunction<TSharedRef<SActorModifierCoreEditorProfiler>(TSharedPtr<FActorModifierCoreProfiler>)>* WidgetFunction = ModifierProfilerWidgets.Find(InProfiler->GetProfilerType()))
	{
		return (*WidgetFunction)(InProfiler);
	}

	return SNew(SActorModifierCoreEditorProfiler, InProfiler);
}

bool UActorModifierCoreEditorSubsystem::RegisterModifierCategory(const FActorModifierCoreEditorCategoryMetadata& InCategoryMetadata)
{
	if (InCategoryMetadata.Name.IsNone())
	{
		return false;
	}

	if (ModifierCategories.Contains(InCategoryMetadata.Name))
	{
		return false;
	}

	ModifierCategories.Emplace(InCategoryMetadata.Name, MakeShared<const FActorModifierCoreEditorCategoryMetadata>(InCategoryMetadata));

	return true;
}

TSharedPtr<const FActorModifierCoreEditorCategoryMetadata> UActorModifierCoreEditorSubsystem::FindModifierCategory(FName InCategoryIdentifier) const
{
	if (const TSharedRef<const FActorModifierCoreEditorCategoryMetadata>* Category = ModifierCategories.Find(InCategoryIdentifier))
	{
		return *Category;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
