// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "ToolMenus.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class UActorModifierCoreBase;
class UObject;

/** Menu type available */
enum class EActorModifierCoreEditorMenuType : uint8
{
	/** Menu to add a modifier at the end of the stack */
	Add,
	/** Menu to insert a modifier in the stack */
	InsertBefore,
	InsertAfter,
	/** Menu to delete a modifier in the stack */
	Delete,
	/** Menu to move a modifier in the stack */
	Move,
	/** Menu to enable or disable modifiers or stack */
	Enable,
	Disable
};

/** Menu customization options */
struct ACTORMODIFIERCOREEDITOR_API FActorModifierCoreEditorMenuOptions
{
	explicit FActorModifierCoreEditorMenuOptions(EActorModifierCoreEditorMenuType InMenuType)
		: MenuType(InMenuType)
	{};

	FActorModifierCoreEditorMenuOptions& CreateSubMenu(bool bInCreateSubMenu);
	FActorModifierCoreEditorMenuOptions& UseTransact(bool bInUseTransact);
	FActorModifierCoreEditorMenuOptions& FireNotification(bool bInFireNotification);

	EActorModifierCoreEditorMenuType GetMenuType() const { return MenuType; }
	bool ShouldCreateSubMenu() const { return bCreateSubMenu; }
	bool ShouldTransact() const { return bUseTransact; }
	bool ShouldFireNotification() const { return bFireNotification; }

private:
	/** What type of menu should be generated */
	EActorModifierCoreEditorMenuType MenuType = EActorModifierCoreEditorMenuType::Add;

	/** Create a sub-menu to display actions instead of filling on the same level
	 * with submenu : Root/Action/Choices... without : Root/Choices...
	 */
	bool bCreateSubMenu = true;

	/** Create a transaction for actions performed using the menu */
	bool bUseTransact = true;

	/** Should we fire a notification when an action is triggered */
	bool bFireNotification = true;
};

/** Used to cache context results in case you call menu function with different menu options */
struct ACTORMODIFIERCOREEDITOR_API FActorModifierCoreEditorMenuContext
{
	explicit FActorModifierCoreEditorMenuContext(const TSet<TWeakObjectPtr<UObject>>& InContextObjects);

	const TSet<AActor*>& GetContextActors() const
	{
		return ContextActors;
	}

	const TSet<UActorModifierCoreBase*>& GetContextModifiers() const
	{
		return ContextModifiers;
	}

	const TSet<UActorModifierCoreBase*>& GetContextStacks() const
	{
		return ContextStacks;
	}

	bool IsEmpty() const;
	bool ContainsAnyActor() const;
	bool ContainsAnyModifier() const;
	bool ContainsAnyStack() const;
	bool ContainsOnlyModifier() const;
	bool ContainsNonEmptyStack() const;
	bool ContainsDisabledModifier() const;
	bool ContainsDisabledStack() const;
	bool ContainsEnabledModifier() const;
	bool ContainsEnabledStack() const;

protected:
	TSet<AActor*> ContextActors;
	TSet<UActorModifierCoreBase*> ContextModifiers;
	TSet<UActorModifierCoreBase*> ContextStacks;
};

/** Used internally to group menu data together */
struct FActorModifierCoreEditorMenuData
{
	FActorModifierCoreEditorMenuData(const FActorModifierCoreEditorMenuContext& InContext, const FActorModifierCoreEditorMenuOptions& InOptions)
		: Context(InContext)
		, Options(InOptions)
	{}

	const FActorModifierCoreEditorMenuContext Context;
	const FActorModifierCoreEditorMenuOptions Options;
};
