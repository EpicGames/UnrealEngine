// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/Profiler/ActorModifierCoreProfiler.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class IMenu;

struct FActorModifierCoreEditorProfilerStat
{
	/** Name of this stat */
	FName Name;

	/** Attribute value text for this stat */
	TAttribute<FText> ValueText = FText::GetEmpty();

	/** Attribute value color for this stat */
	TAttribute<FSlateColor> ValueColor = FLinearColor::White;

	/** Suffix added after value, useful for units */
	FString Suffix;
};

/** Represent a modifier profiler widget, inherit from this class to extend behaviour */
class ACTORMODIFIERCOREEDITOR_API SActorModifierCoreEditorProfiler : public SCompoundWidget
{

public:
	static inline constexpr float Padding = 3.f;

	SLATE_BEGIN_ARGS(SActorModifierCoreEditorProfiler) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FActorModifierCoreProfiler> InProfiler);

protected:
	FActorModifierCoreEditorProfilerStat* AddProfilerStat(const FName& InName);
	FActorModifierCoreEditorProfilerStat* GetProfilerStat(const FName& InName);
	virtual void OnProfilerStatAdded(FActorModifierCoreEditorProfilerStat& InStat);

	virtual void SetupProfilerStats(TSharedPtr<FActorModifierCoreProfiler> InProfiler);
	virtual void ScanPropertiesStats();
	virtual void HandlePropertyStat(FProperty* InProperty, uint8* InValuePtr);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void ShowContextMenu(const FVector2D& InPosition);
	TSharedRef<SWidget> OnGetContextMenuWidget();

	void TogglePinProfilerStat(FName InName);
	void SetPinProfilerStat(const FName& InName, bool bInIsPinned);
	bool IsProfilerStatPinned(FName InName) const;

	EVisibility GetStatVisibility(FName InName) const;
	EVisibility GetProfilerVisibility() const;

private:
	TSharedPtr<IMenu> ContextMenu;
	TMap<FName, FActorModifierCoreEditorProfilerStat> ProfilerStats;
	TWeakPtr<FActorModifierCoreProfiler> ProfilerWeak;
};