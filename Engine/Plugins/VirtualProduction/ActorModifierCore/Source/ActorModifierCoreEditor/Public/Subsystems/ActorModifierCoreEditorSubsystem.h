// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreDefs.h"
#include "Modifiers/ActorModifierCoreEditorMenuDefs.h"
#include "Modifiers/Widgets/SActorModifierCoreEditorProfiler.h"
#include "ToolMenus.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ActorModifierCoreEditorSubsystem.generated.h"

class AActor;
class FActorModifierCoreProfiler;
class UActorModifierCoreBase;
class UActorModifierCoreStack;
class UActorModifierCoreSubsystem;

struct FActorModifierCoreEditorCategoryMetadata
{
	FName Name;
	FText DisplayName;
};

/** Singleton class that handles editor operations for modifiers */
UCLASS()
class UActorModifierCoreEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

	friend class SActorModifierCoreEditorProfiler;

public:
	UActorModifierCoreEditorSubsystem();

	ACTORMODIFIERCOREEDITOR_API static UActorModifierCoreEditorSubsystem* Get();

	/** Fills a menu based on context objects and menu options, will perform menu action transaction if wanted, returns true if the menu has been modified */
	ACTORMODIFIERCOREEDITOR_API bool FillModifierMenu(UToolMenu* InMenu, const FActorModifierCoreEditorMenuContext& InContext, const FActorModifierCoreEditorMenuOptions& InMenuOptions) const;

	/** Register a profiler widget based on a modifier profiler class */
	template<typename InProfilerClass, typename = typename TEnableIf<TIsDerivedFrom<InProfilerClass, FActorModifierCoreProfiler>::Value>::Type
		, typename InWidgetClass, typename = typename TEnableIf<TIsDerivedFrom<InWidgetClass, SActorModifierCoreEditorProfiler>::Value>::Type>
	void RegisterProfilerWidget()
	{
		const FName ProfilerType = GetGeneratedTypeName<InProfilerClass>();
		ModifierProfilerWidgets.Add(ProfilerType, [](TSharedPtr<FActorModifierCoreProfiler> InProfiler)
		{
			return SNew(InWidgetClass, InProfiler);
		});
	}

	/** Unregister a modifier profiler widget */
	template<typename InProfilerClass, typename = typename TEnableIf<TIsDerivedFrom<InProfilerClass, FActorModifierCoreProfiler>::Value>::Type>
	void UnregisterProfilerWidget()
	{
		const FName ProfilerType = GetGeneratedTypeName<InProfilerClass>();
		ModifierProfilerWidgets.Remove(ProfilerType);
	}

	/** Creates a profiler widget for a modifier profiler, returns default widget if none was registered */
	TSharedPtr<SActorModifierCoreEditorProfiler> CreateProfilerWidget(TSharedPtr<FActorModifierCoreProfiler> InProfiler);

	/** Register a category for modifier */
	ACTORMODIFIERCOREEDITOR_API bool RegisterModifierCategory(const FActorModifierCoreEditorCategoryMetadata& InCategoryMetadata);

	/** Finds a category for modifier */
	ACTORMODIFIERCOREEDITOR_API TSharedPtr<const FActorModifierCoreEditorCategoryMetadata> FindModifierCategory(FName InCategoryIdentifier) const;

protected:
	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	//~ End USubsystem

	/** Modifier profiler pinned stats based on profiler type */
	TMap<FName, TSet<FName>> ModifierProfilerStats;

	/** Modifier profiler widgets */
	TMap<FName, TFunction<TSharedRef<SActorModifierCoreEditorProfiler>(TSharedPtr<FActorModifierCoreProfiler>)>> ModifierProfilerWidgets;

	/** Registered modifier category for menus and widgets */
	TMap<FName, TSharedRef<const FActorModifierCoreEditorCategoryMetadata>> ModifierCategories;

	/** Runtime subsystem for modifier factories */
	TWeakObjectPtr<UActorModifierCoreSubsystem> EngineSubsystem = nullptr;
};
